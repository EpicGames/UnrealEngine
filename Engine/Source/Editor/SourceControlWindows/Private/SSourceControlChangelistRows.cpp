// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSourceControlChangelistRows.h"

#include "ComponentReregisterContext.h"
#include "FileHelpers.h"
#include "ISourceControlModule.h"
#include "PackageTools.h"
#include "PackageSourceControlHelper.h"
#include "SourceControlHelpers.h"
#include "UncontrolledChangelistsModule.h"
#include "SourceControlOperations.h"
#include "UnsavedAssetsTrackerModule.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Algo/Transform.h"
#include "Containers/Ticker.h"
#include "RevisionControlStyle/RevisionControlStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/StarshipCoreStyle.h"
#include "Misc/ComparisonUtility.h"
#include "Misc/MessageDialog.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

#define LOCTEXT_NAMESPACE "SourceControlChangelistRow"

namespace
{
	bool CheckoutAndSavePackages(const TArray<FString>& Files)
	{
		FPackageSourceControlHelper PackageHelper;
		TArray<FString> PackageNames;
		PackageNames.Reserve(Files.Num());
		for (const FString& Filename : Files)
		{
			PackageNames.Add(UPackageTools::FilenameToPackageName(Filename));
		}

		if (PackageHelper.Checkout(PackageNames))
		{
			TArray<UPackage*> Packages;
			Packages.Reserve(PackageNames.Num());
			{
				for (const FString& PackageName : PackageNames)
				{
					UPackage* Package = FindPackage(nullptr, *PackageName);
					if (ensure(Package))
					{
						Packages.Add(Package);
					}
					else
					{
						Packages.Add(UPackageTools::LoadPackage(PackageName));
					}
				}
			}
			constexpr bool bOnlyDirty = false;
			return UEditorLoadingAndSavingUtils::SavePackages(Packages, bOnlyDirty);
		}
		return false;
	}
}

FName SourceControlFileViewColumn::CheckBox::Id() { return TEXT("CheckBox"); }

FName SourceControlFileViewColumn::Icon::Id() { return TEXT("Icon"); }
FText SourceControlFileViewColumn::Icon::GetDisplayText() {return LOCTEXT("Name_Icon", "Revision Control Status"); }
FText SourceControlFileViewColumn::Icon::GetToolTipText() { return LOCTEXT("Icon_Column_Tooltip", "Displays the asset/file status"); }

FName SourceControlFileViewColumn::Shelve::Id() { return TEXT("Shelve"); }
FText SourceControlFileViewColumn::Shelve::GetDisplayText() { return LOCTEXT("Name_Shelve", "Revision Control Shelve"); }
FText SourceControlFileViewColumn::Shelve::GetToolTipText() { return LOCTEXT("Shelve_Column_Tooltip", "Displays the shelve status"); }

FName SourceControlFileViewColumn::Name::Id() { return TEXT("Name"); }
FText SourceControlFileViewColumn::Name::GetDisplayText() { return LOCTEXT("Name_Column", "Name"); }
FText SourceControlFileViewColumn::Name::GetToolTipText() { return LOCTEXT("Name_Column_Tooltip", "Displays the asset/file name"); }

FName SourceControlFileViewColumn::Path::Id() { return TEXT("Path"); }
FText SourceControlFileViewColumn::Path::GetDisplayText() { return LOCTEXT("Path_Column", "Path"); }
FText SourceControlFileViewColumn::Path::GetToolTipText() { return LOCTEXT("Path_Column_Tooltip", "Displays the asset/file path"); }

FName SourceControlFileViewColumn::Type::Id() { return TEXT("Type"); }
FText SourceControlFileViewColumn::Type::GetDisplayText() { return LOCTEXT("Type_Column", "Type"); }
FText SourceControlFileViewColumn::Type::GetToolTipText() { return LOCTEXT("Type_Column_Tooltip", "Displays the asset type"); }

FName SourceControlFileViewColumn::LastModifiedTimestamp::Id() { return TEXT("LastModifiedTimestamp"); }
FText SourceControlFileViewColumn::LastModifiedTimestamp::GetDisplayText() {return LOCTEXT("LastModifiedTimestamp_Column", "Last Saved"); }
FText SourceControlFileViewColumn::LastModifiedTimestamp::GetToolTipText() { return LOCTEXT("LastMofiedTimestamp_Column_Tooltip", "Displays the last time the file/asset was saved on user hard drive"); }

FName SourceControlFileViewColumn::CheckedOutByUser::Id() { return TEXT("CheckedOutByUser"); }
FText SourceControlFileViewColumn::CheckedOutByUser::GetDisplayText() { return LOCTEXT("CheckedOutByUser_Column", "User"); }
FText SourceControlFileViewColumn::CheckedOutByUser::GetToolTipText() { return LOCTEXT("CheckedOutByUser_Column_Tooltip", "Displays the other user(s) that checked out the file/asset, if any"); }

FName SourceControlFileViewColumn::Changelist::Id() { return TEXT("Changelist"); }
FText SourceControlFileViewColumn::Changelist::GetDisplayText() { return LOCTEXT("Changelist_Column", "Changelist"); }
FText SourceControlFileViewColumn::Changelist::GetToolTipText() { return LOCTEXT("Changelist_Column_Tooltip", "Displays the changelist the asset/file belongs to, if any"); }

FName SourceControlFileViewColumn::Dirty::Id() { return TEXT("Dirty"); }
FText SourceControlFileViewColumn::Dirty::GetDisplayText() { return LOCTEXT("Dirty_Column", "Unsaved"); }
FText SourceControlFileViewColumn::Dirty::GetToolTipText() { return LOCTEXT("Dirty_Column_Tooltip", "Displays whether the asset/file has unsaved changes"); }

FName SourceControlFileViewColumn::Discard::Id() { return TEXT("Discard"); }
FText SourceControlFileViewColumn::Discard::GetDisplayText() { return LOCTEXT("Discard_Column", "Discard Unsaved Changes"); }
FText SourceControlFileViewColumn::Discard::GetToolTipText() { return LOCTEXT("Discard_Column_Tooltip", "Provides option to discard unsaved changes to an asset/file"); }

namespace SourceControlFileViewColumn
{
	TFunction<int32(const IFileViewTreeItem&, const IFileViewTreeItem&)> GetColumnComparer(FName ColumnId, EPathFlags PathFlags)
	{
		if (ColumnId == SourceControlFileViewColumn::CheckBox::Id())
		{
			return [](const IFileViewTreeItem& Lhs, const IFileViewTreeItem& Rhs)
			{
				ECheckBoxState LhsVal = Lhs.GetCheckBoxState();
				ECheckBoxState RhsVal = Rhs.GetCheckBoxState();
				return LhsVal < RhsVal ? -1 : (LhsVal == RhsVal ? 0 : 1);
			};
		}
		else if (ColumnId == SourceControlFileViewColumn::Icon::Id())
		{
			return [](const IFileViewTreeItem& Lhs, const IFileViewTreeItem& Rhs)
			{
				int32 LhsVal = Lhs.GetIconSortingPriority();
				int32 RhsVal = Rhs.GetIconSortingPriority();
				return LhsVal < RhsVal ? -1 : (LhsVal == RhsVal ? 0 : 1);
			};
		}
		else if (ColumnId == SourceControlFileViewColumn::Shelve::Id())
		{
			return [](const IFileViewTreeItem& Lhs, const IFileViewTreeItem& Rhs)
			{
				const IChangelistTreeItem::TreeItemType LhsTreeItemType = Lhs.GetTreeItemType();
				const IChangelistTreeItem::TreeItemType RhsTreeItemType = Rhs.GetTreeItemType();

				return LhsTreeItemType < RhsTreeItemType ? -1 : (LhsTreeItemType == RhsTreeItemType ? 0 : 1);
			};
		}
		else if (ColumnId == SourceControlFileViewColumn::Name::Id())
		{
			return [](const IFileViewTreeItem& Lhs, const IFileViewTreeItem& Rhs)
			{
				return UE::ComparisonUtility::CompareNaturalOrder(*Lhs.GetName(), *Rhs.GetName());
			};
		}
		else if (ColumnId == SourceControlFileViewColumn::Path::Id())
		{
			if (PathFlags == EPathFlags::ShowingVersePath)
			{
				return [](const IFileViewTreeItem& Lhs, const IFileViewTreeItem& Rhs)
				{
					if (Lhs.GetVersePath().IsValid() && Rhs.GetVersePath().IsValid())
					{
						return Lhs.GetVersePath().Compare(Rhs.GetVersePath());
					}
					else if (Lhs.GetVersePath().IsValid())
					{
						return Lhs.GetVersePath().ToString().Compare(Rhs.GetPath());
					}
					else if (Rhs.GetVersePath().IsValid())
					{
						return Lhs.GetPath().Compare(Rhs.GetVersePath().ToString());
					}
					return Lhs.GetPath().Compare(Rhs.GetPath());
				};
			}

			return [](const IFileViewTreeItem& Lhs, const IFileViewTreeItem& Rhs)
			{
				return Lhs.GetPath().Compare(Rhs.GetPath());
			};
		}
		else if (ColumnId == SourceControlFileViewColumn::Type::Id())
		{
			return [](const IFileViewTreeItem& Lhs, const IFileViewTreeItem& Rhs)
			{
				return FCString::Stricmp(*Lhs.GetType(), *Rhs.GetType());
			};
		}
		else if (ColumnId == SourceControlFileViewColumn::LastModifiedTimestamp::Id())
		{
			return [](const IFileViewTreeItem& Lhs, const IFileViewTreeItem& Rhs)
			{
				const FDateTime& LhsVal = Lhs.GetLastModifiedDateTime();
				const FDateTime& RhsVal = Rhs.GetLastModifiedDateTime();
				return LhsVal < RhsVal ? -1 : (LhsVal == RhsVal ? 0 : 1);
			};
		}
		else if (ColumnId == SourceControlFileViewColumn::CheckedOutByUser::Id())
		{
			return [](const IFileViewTreeItem& Lhs, const IFileViewTreeItem& Rhs)
			{
				return FCString::Stricmp(*Lhs.GetCheckedOutBy(), *Rhs.GetCheckedOutBy());
			};
		}
		else
		{
			checkNoEntry();
			return [](const IFileViewTreeItem& Lhs, const IFileViewTreeItem& Rhs)
			{
				return &Lhs < &Rhs ? -1 : (&Lhs == &Rhs ? 0 : 1);
			};
		};
	}

	TFunction<bool(const IFileViewTreeItem&, const IFileViewTreeItem&)> GetSortPredicate(EColumnSortMode::Type SortMode, FName ColumnId, EPathFlags PathFlags)
	{
		int32 Sign;
		if (SortMode == EColumnSortMode::Ascending)
		{
			Sign = 1;
		}
		else if (SortMode == EColumnSortMode::Descending)
		{
			Sign = -1;
		}
		else
		{
			return {};
		}

		TArray<TFunction<int32(const IFileViewTreeItem&, const IFileViewTreeItem&)>> ColumnComparers;
		ColumnComparers.Reserve(3);

		ColumnComparers.Add(GetColumnComparer(ColumnId, PathFlags));

		if (ColumnId != Path::Id())
		{
			// Use name and path as tie breakers.
			if (ColumnId != Name::Id())
			{
				ColumnComparers.Add(GetColumnComparer(Name::Id(), PathFlags));
			}
			ColumnComparers.Add(GetColumnComparer(Path::Id(), PathFlags));
		}

		return [Sign, ColumnComparers = MoveTemp(ColumnComparers)](const IFileViewTreeItem& Lhs, const IFileViewTreeItem& Rhs)
		{
			for (const TFunction<int32(const IFileViewTreeItem&, const IFileViewTreeItem&)>& ColumnComparer : ColumnComparers)
			{
				const int32 Result = ColumnComparer(Lhs, Rhs);
				if (Result != 0)
				{
					return Sign * Result < 0;
				}
			}
			return false;
		};
	}
}

FText FormatChangelistFileCountText(int32 DisplayedCount, int32 TotalCount)
{
	return DisplayedCount == TotalCount ?
		FText::Format(INVTEXT("({0})"), TotalCount) :
		FText::Format(LOCTEXT("FilterNum", "({0} out of {1})"), DisplayedCount, TotalCount);
}

void SChangelistTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner)
{
	TreeItem = static_cast<FChangelistTreeItem*>(InArgs._TreeItemToVisualize.Get());
	OnPostDrop = InArgs._OnPostDrop;

	SetToolTipText(GetChangelistDescriptionText());

	STableRow<FChangelistTreeItemPtr>::Construct(
		STableRow<FChangelistTreeItemPtr>::FArguments()
		.Style(FAppStyle::Get(), "TableView.Row")
		.Content()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot() // Icon
			.AutoWidth()
			[
				SNew(SImage)
				.Image_Lambda([this]()
				{
					return (TreeItem != nullptr) ?FAppStyle::GetBrush(TreeItem->ChangelistState->GetSmallIconName()) : FAppStyle::GetBrush("SourceControl.Changelist");
				})
			]
			+SHorizontalBox::Slot() // Changelist number.
			.Padding(2, 0, 0, 0)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SChangelistTableRow::GetChangelistText)
				.HighlightText(InArgs._HighlightText)
			]
			+SHorizontalBox::Slot() // Files count.
			.Padding(4, 0, 4, 0)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text_Lambda([this]()
				{
					// Check if the 'Shelved Files' node is currently linked to the  tree view. (not filtered out).
					return FormatChangelistFileCountText(TreeItem->ShelvedChangelistItem->GetParent() ? TreeItem->GetChildren().Num() - 1 : TreeItem->GetChildren().Num(), TreeItem->GetFileCount());
				})
			]
			+SHorizontalBox::Slot() // Description.
			.Padding(2, 0, 0, 0)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(this, &SChangelistTableRow::GetChangelistDescriptionSingleLineText)
				.HighlightText(InArgs._HighlightText)
			]
		], InOwner);
}

void SChangelistTableRow::PopulateSearchString(const FChangelistTreeItem& Item, TArray<FString>& OutStrings)
{
	OutStrings.Emplace(Item.GetDisplayText().ToString()); // The changelist number
	OutStrings.Emplace(Item.GetDescriptionText().ToString());   // The changelist description.
}

FText SChangelistTableRow::GetChangelistText() const
{
	return TreeItem->GetDisplayText();
}

FText SChangelistTableRow::GetChangelistDescriptionText() const
{
	return TreeItem->GetDescriptionText();
}

FText SChangelistTableRow::GetChangelistDescriptionSingleLineText() const
{
	using namespace SSourceControlCommon;
	return GetSingleLineChangelistDescription(TreeItem->GetDescriptionText(), ESingleLineFlags::NewlineConvertToSpace);
}

FReply SChangelistTableRow::OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	TSharedPtr<FSCCFileDragDropOp> DropOperation = InDragDropEvent.GetOperationAs<FSCCFileDragDropOp>();
	if (DropOperation.IsValid())
	{
		FSourceControlChangelistPtr DestChangelist = TreeItem->ChangelistState->GetChangelist();
		check(DestChangelist.IsValid());

		// NOTE: The UI don't show 'source controlled files' and 'uncontrolled files' at the same time. User cannot select and drag/drop both file types at the same time.
		if (!DropOperation->Files.IsEmpty())
		{
			TArray<FString> Files;
			Algo::Transform(DropOperation->Files, Files, [](const FSourceControlStateRef& State) { return State->GetFilename(); });

			SSourceControlCommon::ExecuteChangelistOperationWithSlowTaskWrapper(LOCTEXT("Dropping_Files_On_Changelist", "Moving file(s) to the selected changelist..."), [&]()
			{
				ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
				SourceControlProvider.Execute(ISourceControlOperation::Create<FMoveToChangelist>(), DestChangelist, Files, EConcurrency::Synchronous, FSourceControlOperationComplete::CreateLambda(
					[](const TSharedRef<ISourceControlOperation>& Operation, ECommandResult::Type InResult)
					{
						if (InResult == ECommandResult::Succeeded)
						{
							SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Drop_Files_On_Changelist_Succeeded", "File(s) successfully moved to the selected changelist."), SNotificationItem::CS_Success);
						}
						else if (InResult == ECommandResult::Failed)
						{
							SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Drop_Files_On_Changelist_Failed", "Failed to move the file(s) to the selected changelist."), SNotificationItem::CS_Fail);
						}
					}));
			});
		}
		else if (!DropOperation->UncontrolledFiles.IsEmpty())
		{
			// NOTE: This function does several operations that can fails but we don't get feedback.
			SSourceControlCommon::ExecuteUncontrolledChangelistOperationWithSlowTaskWrapper(LOCTEXT("Dropping_Uncontrolled_Files_On_Changelist", "Moving uncontrolled file(s) to the selected changelist..."), 
				[&DropOperation, &DestChangelist]()
			{
				FUncontrolledChangelistsModule::Get().MoveFilesToControlledChangelist(DropOperation->UncontrolledFiles, DestChangelist, SSourceControlCommon::OpenConflictDialog);
					
				// TODO: Fix MoveFilesToControlledChangelist() to report the possible errors and display a notification.
			});

			OnPostDrop.ExecuteIfBound();
		}
		else if (!DropOperation->OfflineFiles.IsEmpty())
		{
			const TArray<FString>& Files = DropOperation->OfflineFiles;
			SSourceControlCommon::ExecuteChangelistOperationWithSlowTaskWrapper(LOCTEXT("Dropping_Files_On_Changelist", "Moving file(s) to the selected changelist..."), [&]()
				{
					ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

					if (!CheckoutAndSavePackages(Files))
					{
						return;
					}

					SourceControlProvider.Execute(ISourceControlOperation::Create<FMoveToChangelist>(), DestChangelist, Files, EConcurrency::Synchronous, FSourceControlOperationComplete::CreateLambda(
						[](const TSharedRef<ISourceControlOperation>& Operation, ECommandResult::Type InResult)
						{
							if (InResult == ECommandResult::Succeeded)
							{
								SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Drop_Files_On_Changelist_Succeeded", "File(s) successfully moved to the selected changelist."), SNotificationItem::CS_Success);
							}
							else if (InResult == ECommandResult::Failed)
							{
								SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Drop_Files_On_Changelist_Failed", "Failed to move the file(s) to the selected changelist."), SNotificationItem::CS_Fail);
							}
						}));
				});
		}
	}

	return FReply::Handled();
}


void SUncontrolledChangelistTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner)
{
	TreeItem = static_cast<FUncontrolledChangelistTreeItem*>(InArgs._TreeItemToVisualize.Get());
	OnPostDrop = InArgs._OnPostDrop;

		const FSlateBrush* IconBrush = (TreeItem != nullptr) ?
			FAppStyle::GetBrush(TreeItem->UncontrolledChangelistState->GetSmallIconName()) :
			FAppStyle::GetBrush("SourceControl.Changelist");

	SetToolTipText(GetChangelistText());

	STableRow<FChangelistTreeItemPtr>::Construct(
		STableRow<FChangelistTreeItemPtr>::FArguments()
		.Style(FAppStyle::Get(), "TableView.Row")
		.Content()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(IconBrush)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SUncontrolledChangelistTableRow::GetChangelistText)
				.HighlightText(InArgs._HighlightText)
			]
			+SHorizontalBox::Slot() // Files/Offline file count.
			.Padding(4, 0, 4, 0)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text_Lambda([this]()
				{
					return FormatChangelistFileCountText(TreeItem->GetChildren().Num(), TreeItem->GetFileCount());
				})
			]
		], InOwner);
}

void SUncontrolledChangelistTableRow::PopulateSearchString(const FUncontrolledChangelistTreeItem& Item, TArray<FString>& OutStrings)
{
	OutStrings.Emplace(Item.GetDisplayText().ToString());
}

FText SUncontrolledChangelistTableRow::GetChangelistText() const
{
	return TreeItem->GetDisplayText();
}


FReply SUncontrolledChangelistTableRow::OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent)
{
	TSharedPtr<FSCCFileDragDropOp> Operation = InDragDropEvent.GetOperationAs<FSCCFileDragDropOp>();

	if (Operation.IsValid())
	{
		if (Operation->OfflineFiles.IsEmpty())
		{
			SSourceControlCommon::ExecuteUncontrolledChangelistOperationWithSlowTaskWrapper(LOCTEXT("Drag_File_To_Uncontrolled_Changelist", "Moving file(s) to the selected uncontrolled changelists..."),
				[this, &Operation]()
				{
					FUncontrolledChangelistsModule::Get().MoveFilesToUncontrolledChangelist(Operation->Files, Operation->UncontrolledFiles, TreeItem->UncontrolledChangelistState->Changelist);
				});
		}
		// Drop unsaved assets (offline files)
		else
		{
			const TArray<FString>& Files = Operation->OfflineFiles;
			if (!CheckoutAndSavePackages(Files))
			{
				return FReply::Unhandled();
			}

			SSourceControlCommon::ExecuteUncontrolledChangelistOperationWithSlowTaskWrapper(LOCTEXT("Drag_File_To_Uncontrolled_Changelist", "Moving file(s) to the selected uncontrolled changelists..."),
				[this, &Files]()
				{
					FUncontrolledChangelistsModule::Get().MoveFilesToUncontrolledChangelist(Files, TreeItem->UncontrolledChangelistState->Changelist);
				});
		}

		OnPostDrop.ExecuteIfBound();
	}

	return FReply::Handled();
}

void SUnsavedAssetsTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner)
{
	STableRow<FChangelistTreeItemPtr>::Construct(
		STableRow<FChangelistTreeItemPtr>::FArguments()
		.Style(FAppStyle::Get(), "TableView.Row")
		.Content()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Assets.Unsaved"))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("SourceControl_Unsaved", "Unsaved"))
			]
			+SHorizontalBox::Slot() // Files/Offline file count.
			.Padding(4, 0, 4, 0)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text_Lambda([] { return FText::Format(FText::FromString("({0})"), FUnsavedAssetsTrackerModule::Get().GetUnsavedAssetNum()); })
			]
		], InOwner);
}


void SFileTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner)
{
	TreeItem = static_cast<FFileTreeItem*>(InArgs._TreeItemToVisualize.Get());

	HighlightText = InArgs._HighlightText;

	PathFlags = InArgs._PathFlags;

	FSuperRowType::FArguments Args = FSuperRowType::FArguments()
		.OnDragDetected(InArgs._OnDragDetected)
		.ShowSelection(true);
	FSuperRowType::Construct(Args, InOwner);
}

TSharedRef<SWidget> SFileTableRow::GenerateWidgetForColumn(const FName& ColumnId)
{
	auto ConstructPaddedWidget =
		[](const TSharedRef<SWidget>& Widget)
		{
			TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

			HorizontalBox->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(8, 0, 0, 0)
				[
					Widget
				];

			return HorizontalBox;
		};

	if (ColumnId == SourceControlFileViewColumn::CheckBox::Id())
	{
		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(FMargin(10.0f, 3.0f, 6.0f, 3.0f))
			[
				SNew(SCheckBox)
				.IsChecked(this, &SFileTableRow::GetCheckBoxState)
				.OnCheckStateChanged(this, &SFileTableRow::SetCheckBoxState)
			];
	}
	else if (ColumnId == SourceControlFileViewColumn::Icon::Id())
	{
		return SNew(SBox)
			.WidthOverride(16) // Small Icons are usually 16x16
			.HAlign(HAlign_Center)
			[
				SSourceControlCommon::GetSCCStatusWidget(TreeItem->FileState)
			];
	}
	else if (ColumnId == SourceControlFileViewColumn::Shelve::Id())
	{
		return SNew(SBox)
			.WidthOverride(16) // Small Icons are usually 16x16
			.HAlign(HAlign_Center)
			[
				SSourceControlCommon::GetSCCShelveWidget(TreeItem->IsShelved())
			];
	}
	else if (ColumnId == SourceControlFileViewColumn::Name::Id())
	{
		TSharedPtr<SWidget> Widget = SNew(STextBlock)
			.Text(this, &SFileTableRow::GetDisplayName)
			.ToolTipText(this, &SFileTableRow::GetDisplayName)
			.HighlightText(HighlightText);
		return ConstructPaddedWidget(Widget.ToSharedRef());
	}
	else if (ColumnId == SourceControlFileViewColumn::Path::Id())
	{
		TSharedPtr<SWidget> Widget = SNew(STextBlock)
			.Text(this, &SFileTableRow::GetDisplayPath)
			.ToolTipText(this, &SFileTableRow::GetFilename)
			.HighlightText(HighlightText);
		return ConstructPaddedWidget(Widget.ToSharedRef());
	}
	else if (ColumnId == SourceControlFileViewColumn::Type::Id())
	{
		TSharedPtr<SWidget> Widget = SNew(STextBlock)
			.Text(this, &SFileTableRow::GetDisplayType)
			.ToolTipText(this, &SFileTableRow::GetDisplayType)
			.ColorAndOpacity(this, &SFileTableRow::GetDisplayColor)
			.HighlightText(HighlightText);
		return ConstructPaddedWidget(Widget.ToSharedRef());
	}
	else if (ColumnId == SourceControlFileViewColumn::LastModifiedTimestamp::Id())
	{
		TSharedPtr<SWidget> Widget = SNew(STextBlock)
			.ToolTipText(this, &SFileTableRow::GetLastModifiedTimestamp)
			.Text(this, &SFileTableRow::GetLastModifiedTimestamp);
		return ConstructPaddedWidget(Widget.ToSharedRef());
	}
	else if (ColumnId == SourceControlFileViewColumn::CheckedOutByUser::Id())
	{
		TSharedPtr<SWidget> Widget = SNew(STextBlock)
			.ToolTipText(this, &SFileTableRow::GetCheckedOutByUser)
			.Text(this, &SFileTableRow::GetCheckedOutByUser);
		return ConstructPaddedWidget(Widget.ToSharedRef());
	}
	else
	{
		return SNullWidget::NullWidget;
	}
}

void SFileTableRow::PopulateSearchString(const FFileTreeItem& Item, SourceControlFileViewColumn::EPathFlags PathFlags, TArray<FString>& OutStrings)
{
	OutStrings.Emplace(Item.GetAssetName().ToString()); // Name
	if (EnumHasAnyFlags(PathFlags, SourceControlFileViewColumn::EPathFlags::ShowingVersePath) && Item.GetVersePath().IsValid())
	{
		OutStrings.Emplace(Item.GetAssetVersePath().ToString());
	}
	else
	{
		OutStrings.Emplace(Item.GetAssetPath().ToString()); // Path
	}
	OutStrings.Emplace(Item.GetAssetType().ToString()); // Type
	OutStrings.Emplace(Item.GetLastModifiedTimestamp().ToString());
	OutStrings.Emplace(Item.GetCheckedOutByUser().ToString());
}

ECheckBoxState SFileTableRow::GetCheckBoxState() const
{
	return TreeItem->GetCheckBoxState();
}

void SFileTableRow::SetCheckBoxState(ECheckBoxState NewState)
{
	TreeItem->SetCheckBoxState(NewState);
}

FText SFileTableRow::GetDisplayName() const
{
	return TreeItem->GetAssetName();
}

FText SFileTableRow::GetFilename() const
{
	return TreeItem->GetFileName();
}

FText SFileTableRow::GetDisplayPath() const
{
	if (EnumHasAnyFlags(PathFlags.Get(SourceControlFileViewColumn::EPathFlags::Default), SourceControlFileViewColumn::EPathFlags::ShowingVersePath) && TreeItem->GetVersePath().IsValid())
	{
		return TreeItem->GetAssetVersePath();
	}
	return TreeItem->GetAssetPath();
}

FText SFileTableRow::GetDisplayType() const
{
	return TreeItem->GetAssetType();
}

FSlateColor SFileTableRow::GetDisplayColor() const
{
	return TreeItem->GetAssetTypeColor();
}

FText SFileTableRow::GetLastModifiedTimestamp() const
{
	return TreeItem->GetLastModifiedTimestamp();
}

FText SFileTableRow::GetCheckedOutByUser() const
{
	return TreeItem->GetCheckedOutByUser();
}

void SFileTableRow::OnDragEnter(FGeometry const& InGeometry, FDragDropEvent const& InDragDropEvent)
{
	TSharedPtr<FDragDropOperation> DragOperation = InDragDropEvent.GetOperation();
	DragOperation->SetCursorOverride(EMouseCursor::SlashedCircle);
}

void SFileTableRow::OnDragLeave(FDragDropEvent const& InDragDropEvent)
{
	TSharedPtr<FDragDropOperation> DragOperation = InDragDropEvent.GetOperation();
	DragOperation->SetCursorOverride(EMouseCursor::None);
}


void SShelvedFilesTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	TreeItem = static_cast<FShelvedChangelistTreeItem*>(InArgs._TreeItemToVisualize.Get());

	STableRow<FChangelistTreeItemPtr>::Construct(
		STableRow<FChangelistTreeItemPtr>::FArguments()
		.Content()
		[
			SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(5, 0, 0, 0)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("SourceControl.ShelvedChangelist"))
				]
				+SHorizontalBox::Slot()
				.Padding(2.0f, 1.0f)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text_Lambda([this](){ return TreeItem->GetDisplayText(); })
					.HighlightText(InArgs._HighlightText)
				]
				+SHorizontalBox::Slot() // Shelved file count.
				.Padding(4, 0, 4, 0)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text_Lambda([this]()
					{
						return FormatChangelistFileCountText(TreeItem->GetChildren().Num(), static_cast<const FChangelistTreeItem*>(TreeItem->GetParent().Get())->GetShelvedFileCount());
					})
				]
		],
		InOwnerTableView);
}

void SShelvedFilesTableRow::PopulateSearchString(const FShelvedChangelistTreeItem& Item, TArray<FString>& OutStrings)
{
	OutStrings.Emplace(Item.GetDisplayText().ToString());
}


void SOfflineFileTableRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner)
{
	TreeItem = static_cast<FOfflineFileTreeItem*>(InArgs._TreeItemToVisualize.Get());
	HighlightText = InArgs._HighlightText;
	PathFlags = InArgs._PathFlags;

	FSuperRowType::FArguments Args = FSuperRowType::FArguments()
		.OnDragDetected(InArgs._OnDragDetected)
		.ShowSelection(true);
	FSuperRowType::Construct(Args, InOwner);
}

TSharedRef<SWidget> SOfflineFileTableRow::GenerateWidgetForColumn(const FName& ColumnId)
{
	auto ConstructPaddedWidget =
		[](const TSharedRef<SWidget>& Widget)
		{
			TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox);

			HorizontalBox->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(8, 0, 0, 0)
				[
					Widget
				];

			return HorizontalBox;
		};

	if (ColumnId == SourceControlFileViewColumn::CheckBox::Id())
	{
		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(FMargin(10.0f, 3.0f, 6.0f, 3.0f))
			[
				SNew(SCheckBox)
				.IsChecked(this, &SOfflineFileTableRow::GetCheckBoxState)
				.OnCheckStateChanged(this, &SOfflineFileTableRow::SetCheckBoxState)
			];
	}
	else if (ColumnId == SourceControlFileViewColumn::Icon::Id())
	{
		return SNew(SBox)
			.WidthOverride(16) // Small Icons are usually 16x16
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush(FName("SourceControl.OfflineFile_Small")))
			];
	}
	if (ColumnId == SourceControlFileViewColumn::Shelve::Id())
	{
		return SNew(SBox)
			.WidthOverride(16) // Small Icons are usually 16x16
			.HAlign(HAlign_Center)
			[
				SNullWidget::NullWidget
			];
	}
	else if (ColumnId == SourceControlFileViewColumn::Name::Id())
	{
		TSharedPtr<SWidget> Widget = SNew(STextBlock)
			.ToolTipText(this, &SOfflineFileTableRow::GetDisplayName)
			.Text(this, &SOfflineFileTableRow::GetDisplayName)
			.HighlightText(HighlightText);
		return ConstructPaddedWidget(Widget.ToSharedRef());
	}
	else if (ColumnId == SourceControlFileViewColumn::Path::Id())
	{
		TSharedPtr<SWidget> Widget = SNew(STextBlock)
			.Text(this, &SOfflineFileTableRow::GetDisplayPath)
			.ToolTipText(this, &SOfflineFileTableRow::GetFilename)
			.HighlightText(HighlightText);
		return ConstructPaddedWidget(Widget.ToSharedRef());
	}
	else if (ColumnId == SourceControlFileViewColumn::Type::Id())
	{
		TSharedPtr<SWidget> Widget = SNew(STextBlock)
			.Text(this, &SOfflineFileTableRow::GetDisplayType)
			.ToolTipText(this, &SOfflineFileTableRow::GetDisplayType)
			.ColorAndOpacity(this, &SOfflineFileTableRow::GetDisplayColor)
			.HighlightText(HighlightText);
		return ConstructPaddedWidget(Widget.ToSharedRef());
	}
	else if (ColumnId == SourceControlFileViewColumn::LastModifiedTimestamp::Id())
	{
		TSharedPtr<SWidget> Widget = SNew(STextBlock)
			.ToolTipText(this, &SOfflineFileTableRow::GetLastModifiedTimestamp)
			.Text(this, &SOfflineFileTableRow::GetLastModifiedTimestamp);
		return ConstructPaddedWidget(Widget.ToSharedRef());
	}
	else if (ColumnId == SourceControlFileViewColumn::CheckedOutByUser::Id())
	{
		TSharedPtr<SWidget> Widget = SNew(STextBlock)
			.Text(FText::GetEmpty());
		return ConstructPaddedWidget(Widget.ToSharedRef());
	}
	else if (ColumnId == SourceControlFileViewColumn::Dirty::Id())
	{
		if (FUnsavedAssetsTrackerModule::Get().IsAssetUnsaved(GetFilename().ToString()))
		{
			return SNew(SBox)
				.WidthOverride(16) // Small Icons are usually 16x16
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(FName("SourceControl.OfflineFile_Small")))
				];
		}
		return SNew(SBox)
			.WidthOverride(16); // Small Icons are usually 16x16
	}
	else if (ColumnId == SourceControlFileViewColumn::Discard::Id())
	{
		const FString& Filename = TreeItem->GetFullPathname();
		if (!FUnsavedAssetsTrackerModule::Get().IsAssetUnsaved(Filename))
		{
			return SNew(SBox)
				.WidthOverride(16); // Small Icons are usually 16x16
		}

		TSharedRef<SWidgetSwitcher> DiscardSwitcher = SNew(SWidgetSwitcher);

		TSharedRef<SImage> DiscardButton = SNew(SImage)
				.DesiredSizeOverride(FVector2D{ 16.0f })
				.Image(FAppStyle::Get().GetBrush("Icons.XCircle"))
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.OnMouseButtonDown_Lambda([this, DiscardSwitcher] (const FGeometry&, const FPointerEvent&) -> FReply
				{
					// Normalize packagenames and filenames
					FString PackageName;
					if (FPackageName::TryConvertFilenameToLongPackageName(TreeItem->GetFullPathname(), PackageName))
					{
						// Validate we have a saved map
						if (UPackage* Package = FindPackage(nullptr, *PackageName))
						{
							UPackage* LevelPackage = Package->GetOutermost();
							if (LevelPackage == GetTransientPackage()
								|| LevelPackage->HasAnyFlags(RF_Transient)
								|| !FPackageName::IsValidLongPackageName(LevelPackage->GetName()))
							{
								FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("DiscardUnsavedChangesSaveMap", "You need to save the level before discarding unsaved changes."));
								return FReply::Handled();
							}
						}
					}
					
					DiscardSwitcher->SetActiveWidgetIndex(1);
					ExecuteOnGameThread(UE_SOURCE_LOCATION,
						[this]
						{
							TArray<FString> Filenames { TreeItem->GetFullPathname() };
							const bool bAllowReloadWorld = true;
							const bool bInteractive = false;
							USourceControlHelpers::ApplyOperationAndReloadPackages(
								Filenames,
								[](const TArray<FString>&) -> bool { return true; },
								bAllowReloadWorld,
								bInteractive
							);
						});
					
					return FReply::Handled();
				});

		DiscardSwitcher->AddSlot()[
			DiscardButton
		];
		DiscardSwitcher->AddSlot()[
			SNew(SCircularThrobber)
			.Radius(7.5f)
		];
		
		return
			SNew(SBox)
			.WidthOverride(16) // Small Icons are usually 16x16
			.Padding(FMargin{1, 0})
			.ToolTipText(LOCTEXT("UnsavedAsset_DiscardChanges", "Discard unsaved changes"))
			[
				DiscardSwitcher
			];
	}
	else
	{
		return SNullWidget::NullWidget;
	}
}

void SOfflineFileTableRow::PopulateSearchString(const FOfflineFileTreeItem& Item, SourceControlFileViewColumn::EPathFlags PathFlags, TArray<FString>& OutStrings)
{
	OutStrings.Emplace(Item.GetDisplayName().ToString()); // Name
	if (EnumHasAnyFlags(PathFlags, SourceControlFileViewColumn::EPathFlags::ShowingVersePath) && Item.GetVersePath().IsValid())
	{
		OutStrings.Emplace(Item.GetDisplayVersePath().ToString());
	}
	else
	{
		OutStrings.Emplace(Item.GetDisplayPath().ToString()); // Path
	}
	OutStrings.Emplace(Item.GetDisplayType().ToString()); // Type
	OutStrings.Emplace(Item.GetLastModifiedTimestamp().ToString());
}

ECheckBoxState SOfflineFileTableRow::GetCheckBoxState() const
{
	return TreeItem->GetCheckBoxState();
}

void SOfflineFileTableRow::SetCheckBoxState(ECheckBoxState NewState)
{
	TreeItem->SetCheckBoxState(NewState);
}

FText SOfflineFileTableRow::GetDisplayName() const
{
	return TreeItem->GetDisplayName();
}

FText SOfflineFileTableRow::GetFilename() const
{
	return TreeItem->GetDisplayFilename();
}

FText SOfflineFileTableRow::GetDisplayPath() const
{
	if (EnumHasAnyFlags(PathFlags.Get(SourceControlFileViewColumn::EPathFlags::Default), SourceControlFileViewColumn::EPathFlags::ShowingVersePath) && TreeItem->GetVersePath().IsValid())
	{
		return TreeItem->GetDisplayVersePath();
	}
	return TreeItem->GetDisplayPath();
}

FText SOfflineFileTableRow::GetDisplayType() const
{
	return TreeItem->GetDisplayType();
}

FSlateColor SOfflineFileTableRow::GetDisplayColor() const
{
	return TreeItem->GetDisplayColor();
}

FText SOfflineFileTableRow::GetLastModifiedTimestamp() const
{
	return TreeItem->GetLastModifiedTimestamp();
}

void SOfflineFileTableRow::OnDragEnter(FGeometry const& InGeometry, FDragDropEvent const& InDragDropEvent)
{
	TSharedPtr<FDragDropOperation> DragOperation = InDragDropEvent.GetOperation();
	DragOperation->SetCursorOverride(EMouseCursor::SlashedCircle);
}

void SOfflineFileTableRow::OnDragLeave(FDragDropEvent const& InDragDropEvent)
{
	TSharedPtr<FDragDropOperation> DragOperation = InDragDropEvent.GetOperation();
	DragOperation->SetCursorOverride(EMouseCursor::None);
}


#undef LOCTEXT_NAMESPACE
