// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Misc/EnumClassFlags.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "Input/DragAndDrop.h"
#include "SSourceControlCommon.h"

/** Lists the unique columns used in the list view displaying controlled/uncontrolled changelist files. */
namespace SourceControlFileViewColumn
{
	/** The checkbox column. */
	namespace CheckBox
	{
		FName Id();
	}

	/** The icon column. */
	namespace Icon
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	}

	/** The shelve column. */
	namespace Shelve
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	}

	/** The file/asset name column. */
	namespace Name
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	}
	
	/** The file/asset path column. */
	namespace Path
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	}

	/** The file/asset type column. */
	namespace Type
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	}

	/** The last time the file/asset was modified column. */
	namespace LastModifiedTimestamp
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	}

	/** The user that has a given file in checkout column. */
	namespace CheckedOutByUser
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	}

	/** "What changelist a given file belongs to" column. */
	namespace Changelist
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	}

	/** "Whether a file is unsaved/dirty" column */
	namespace Dirty
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	}

	/** "Discard unsaved changes" column */
	namespace Discard
	{
		FName Id();
		FText GetDisplayText();
		FText GetToolTipText();
	}

	enum class EPathFlags
	{
		Default = 0,

		ShowingPackageName = 1 << 0,
		ShowingVersePath = 1 << 1,
	};
	ENUM_CLASS_FLAGS(EPathFlags)

	TFunction<int32(const IFileViewTreeItem&, const IFileViewTreeItem&)> GetColumnComparer(FName ColumnId, EPathFlags PathFlags);

	TFunction<bool(const IFileViewTreeItem&, const IFileViewTreeItem&)> GetSortPredicate(EColumnSortMode::Type, FName ColumnId, EPathFlags PathFlags);
}


/** Displays a changed list row (icon, cl number, description) */
class SChangelistTableRow : public STableRow<TSharedPtr<IChangelistTreeItem>>
{
public:
	SLATE_BEGIN_ARGS(SChangelistTableRow)
		: _TreeItemToVisualize()
		, _HighlightText()
		, _OnPostDrop()
	{
	}
		SLATE_ARGUMENT(FChangelistTreeItemPtr, TreeItemToVisualize)
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_EVENT(FSimpleDelegate, OnPostDrop)
	SLATE_END_ARGS()

public:
	/**
	* Construct child widgets that comprise this widget.
	*
	* @param InArgs Declaration from which to construct this widget.
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner);

	FText GetChangelistText() const;
	FText GetChangelistDescriptionText() const;
	FText GetChangelistDescriptionSingleLineText() const;

	static void PopulateSearchString(const FChangelistTreeItem& Item, TArray<FString>& OutStrings);

protected:
	//~ Begin STableRow Interface.
	virtual FReply OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent) override;

private:
	/** The info about the widget that we are visualizing. */
	FChangelistTreeItem* TreeItem;

	/** Delegate invoked once the drag and drop operation finished. */
	FSimpleDelegate OnPostDrop;
};


/** Displays an uncontrolled changed list (icon, cl name, description) */
class SUncontrolledChangelistTableRow : public STableRow<FChangelistTreeItemPtr>
{
public:
	SLATE_BEGIN_ARGS(SUncontrolledChangelistTableRow)
		: _TreeItemToVisualize()
		, _HighlightText()
		, _OnPostDrop()
	{
	}
		SLATE_ARGUMENT(FChangelistTreeItemPtr, TreeItemToVisualize)
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_EVENT(FSimpleDelegate, OnPostDrop)
	SLATE_END_ARGS()

public:
	/**
	* Construct child widgets that comprise this widget.
	*
	* @param InArgs Declaration from which to construct this widget.
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner);

	FText GetChangelistText() const;

	static void PopulateSearchString(const FUncontrolledChangelistTreeItem& Item, TArray<FString>& OutStrings);

protected:
	//~ Begin STableRow Interface.
	virtual FReply OnDrop(const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent) override;

private:
	/** The info about the widget that we are visualizing. */
	FUncontrolledChangelistTreeItem* TreeItem;

	/** Invoked once a drag and drop operation completes. */
	FSimpleDelegate OnPostDrop;
};

/** Displays a block for unsaved assets with icon and count */
class SUnsavedAssetsTableRow : public STableRow<FChangelistTreeItemPtr>
{
public:
	/**
	* Construct child widgets that comprise this widget.
	*
	* @param InArgs Declaration from which to construct this widget.
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner);
};

/** Display information about a file (icon, name, location, type, etc.) */
class SFileTableRow : public SMultiColumnTableRow<FChangelistTreeItemPtr>
{
public:
	SLATE_BEGIN_ARGS(SFileTableRow)
		: _TreeItemToVisualize(nullptr)
		, _HighlightText()
		, _PathFlags(SourceControlFileViewColumn::EPathFlags::Default)
	{
	}
		SLATE_ARGUMENT(FChangelistTreeItemPtr, TreeItemToVisualize)
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_ATTRIBUTE(SourceControlFileViewColumn::EPathFlags, PathFlags)
		SLATE_EVENT(FOnDragDetected, OnDragDetected)
	SLATE_END_ARGS()

public:
	/**
	* Construct child widgets that comprise this widget.
	*
	* @param InArgs Declaration from which to construct this widget.
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner);

	// SMultiColumnTableRow overrides
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	ECheckBoxState GetCheckBoxState() const;
	void SetCheckBoxState(ECheckBoxState NewState);
	FText GetDisplayName() const;
	FText GetFilename() const;
	FText GetDisplayPath() const;
	FText GetDisplayType() const;
	FSlateColor GetDisplayColor() const;
	FText GetLastModifiedTimestamp() const;
	FText GetCheckedOutByUser() const;

	static void PopulateSearchString(const FFileTreeItem& Item, SourceControlFileViewColumn::EPathFlags PathFlags, TArray<FString>& OutStrings);

protected:
	//~ Begin STableRow Interface.
	virtual void OnDragEnter(FGeometry const& InGeometry, FDragDropEvent const& InDragDropEvent) override;
	virtual void OnDragLeave(FDragDropEvent const& InDragDropEvent) override;
	//~ End STableRow Interface.

private:
	/** The info about the widget that we are visualizing. */
	FFileTreeItem* TreeItem;

	/** The text to highlight*/
	TAttribute<FText> HighlightText;

	TAttribute<SourceControlFileViewColumn::EPathFlags> PathFlags;
};


/** Display the shelved files group node. It displays 'Shelved Files (x)' where X is the nubmer of file shelved. */
class SShelvedFilesTableRow : public STableRow<TSharedPtr<IChangelistTreeItem>>
{
public:
	SLATE_BEGIN_ARGS(SShelvedFilesTableRow)
		: _TreeItemToVisualize()
		, _HighlightText()
	{
	}
		SLATE_ARGUMENT(FChangelistTreeItemPtr, TreeItemToVisualize)
		SLATE_ATTRIBUTE(FText, HighlightText)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);
	static void PopulateSearchString(const FShelvedChangelistTreeItem& Item, TArray<FString>& OutStrings);

private:
	FShelvedChangelistTreeItem* TreeItem;

	/** The text to highlight*/
	TAttribute<FText> HighlightText;
};


/** Display information about an offline file (icon, name, location, type, etc.). */
class SOfflineFileTableRow : public SMultiColumnTableRow<FChangelistTreeItemPtr>
{
public:
	SLATE_BEGIN_ARGS(SOfflineFileTableRow)
		: _TreeItemToVisualize()
		, _HighlightText()
		, _PathFlags(SourceControlFileViewColumn::EPathFlags::Default)
	{
	}
		SLATE_ARGUMENT(FChangelistTreeItemPtr, TreeItemToVisualize)
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_ATTRIBUTE(SourceControlFileViewColumn::EPathFlags, PathFlags)
		SLATE_EVENT(FOnDragDetected, OnDragDetected)
	SLATE_END_ARGS()

public:
	/**
	* Construct child widgets that comprise this widget.
	*
	* @param InArgs Declaration from which to construct this widget.
	*/
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner);

	// SMultiColumnTableRow overrides
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	ECheckBoxState GetCheckBoxState() const;
	void SetCheckBoxState(ECheckBoxState NewState);
	FText GetDisplayName() const;
	FText GetFilename() const;
	FText GetDisplayPath() const;
	FText GetDisplayType() const;
	FSlateColor GetDisplayColor() const;
	FText GetLastModifiedTimestamp() const;

	static void PopulateSearchString(const FOfflineFileTreeItem& Item, SourceControlFileViewColumn::EPathFlags PathFlags, TArray<FString>& OutStrings);

protected:
	//~ Begin STableRow Interface.
	virtual void OnDragEnter(FGeometry const& InGeometry, FDragDropEvent const& InDragDropEvent) override;
	virtual void OnDragLeave(FDragDropEvent const& InDragDropEvent) override;
	//~ End STableRow Interface.

private:
	/** The info about the widget that we are visualizing. */
	FOfflineFileTreeItem* TreeItem;

	/** The text to highlight*/
	TAttribute<FText> HighlightText;

	TAttribute<SourceControlFileViewColumn::EPathFlags> PathFlags;
};
