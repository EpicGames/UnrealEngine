// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSourceControlChangelists.h"

#include "Editor.h"
#include "Styling/AppStyle.h"
#include "Algo/Transform.h"
#include "Algo/Count.h"
#include "Logging/MessageLog.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SWidgetSwitcher.h"

#include "IAssetTools.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "ISourceControlWindowsModule.h"
#include "Interfaces/IPluginManager.h"
#include "UncontrolledChangelistsModule.h"
#include "SourceControlOperations.h"
#include "SSourceControlChangelistRows.h"
#include "SSourceControlDescription.h"
#include "SourceControlWindows.h"
#include "SourceControlHelpers.h"
#include "SourceControlFileStatusMonitor.h"
#include "SourceControlPreferences.h"
#include "SourceControlMenuContext.h"
#include "SourceControlSettings.h"
#include "AssetToolsModule.h"
#include "ComponentReregisterContext.h"
#include "FileHelpers.h"
#include "Misc/CString.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/StringOutputDevice.h"
#include "PackageSourceControlHelper.h"
#include "Algo/AnyOf.h"
#include "HAL/PlatformTime.h"
#include "HAL/FileManager.h"
#include "ToolMenus.h"
#include "UnsavedAssetsTrackerModule.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"
#include "SSourceControlSubmit.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "RevisionControlStyle/RevisionControlStyle.h"

#define LOCTEXT_NAMESPACE "SourceControlChangelist"

namespace
{

bool AreUnsavedAssetsPresent()
{
	return FUnsavedAssetsTrackerModule::Get().GetUnsavedAssetNum() > 0;
}
	
/** Returns true if a source control provider is enable and support changeslists. */
bool AreControlledChangelistsEnabled()
{
	return ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().UsesChangelists();
};

/** Returns true if Uncontrolled changelists are enabled. */
bool AreUncontrolledChangelistsEnabled()
{
	return FUncontrolledChangelistsModule::Get().IsEnabled();
};

/** Returns true if there are changelists to display. */
bool AreChangelistsEnabled()
{
	return AreControlledChangelistsEnabled() || AreUncontrolledChangelistsEnabled();
};

/** Returns true if world reloads are allowed. */
bool AreWorldReloadsAllowed()
{
	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("SourceControl.EnableWorldReload")); ensure(CVar))
	{
		return CVar->GetBool();
	}
	else
	{
		return false;
	}
}

bool AnyUncontrolledChangelistContainsFilename(const FString& Filename)
{
	FUncontrolledChangelistsModule& UncontrolledChangelistsModule = FUncontrolledChangelistsModule::Get();

	if (UncontrolledChangelistsModule.IsEnabled())
	{
		TArray<TSharedRef<FUncontrolledChangelistState>> UncontrolledChangelistStates = UncontrolledChangelistsModule.GetChangelistStates();

		for (const TSharedRef<FUncontrolledChangelistState>& UncontrolledChangelistState : UncontrolledChangelistStates)
		{
			if (UncontrolledChangelistState->ContainsFilename(Filename))
			{
				return true;
			}
		}
	}

	return false;
}

/**
 * Returns a new changelist description if needed, appending validation tag.
 * 
 * @param bInValidationResult	The result of the validation step
 * @param InOriginalChangelistDescription	Description of the changelist before modification
 * 
 * @return The new changelist description
 */
FText UpdateChangelistDescriptionToSubmitIfNeeded(const bool bInValidationResult, const FText& InChangelistDescription)
{
	auto GetChangelistValidationTag = []
	{
		return LOCTEXT("ValidationTag", "#changelist validated");
	};

	auto ContainsValidationFlag = [&GetChangelistValidationTag](const FText& InChangelistDescription)
	{
		FString DescriptionString = InChangelistDescription.ToString();
		FString ValidationString = GetChangelistValidationTag().ToString();
		return DescriptionString.Find(ValidationString) != INDEX_NONE;
	};

	auto RemoveValidationFlag = [&GetChangelistValidationTag](const FText& InChangelistDescription)
	{
		FString DescriptionString = InChangelistDescription.ToString();
		FString ValidationString = GetChangelistValidationTag().ToString();
		DescriptionString.ReplaceInline(*ValidationString, TEXT(""));

		return DescriptionString;
	};

	if (bInValidationResult && USourceControlPreferences::IsValidationTagEnabled() && !ContainsValidationFlag(InChangelistDescription))
	{
		FStringOutputDevice Str;

		Str.SetAutoEmitLineTerminator(true);
		Str.Log(InChangelistDescription);
		Str.Log(GetChangelistValidationTag());

		return FText::FromString(Str);
	}

	if (!bInValidationResult && USourceControlPreferences::IsValidationTagEnabled() && ContainsValidationFlag(InChangelistDescription))
	{
		FString NewChangelistDescription = RemoveValidationFlag(InChangelistDescription);
		return FText::FromString(NewChangelistDescription);
	}

	return InChangelistDescription;
}

} // Anonymous namespace


DECLARE_DELEGATE(FOnSearchBoxExpanded)

/** A button that expands a search box below itself when clicked. */
class SExpandableSearchButton : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SExpandableSearchButton)
		: _Style(&FAppStyle::Get().GetWidgetStyle<FSearchBoxStyle>("SearchBox"))
	{}
		/** Search box style (used to match the glass icon) */
		SLATE_STYLE_ARGUMENT(FSearchBoxStyle, Style)

		/** Event fired when the associated search box is made visible */
		SLATE_EVENT(FOnSearchBoxExpanded, OnSearchBoxExpanded)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<SSearchBox> SearchBox)
	{
		OnSearchBoxExpanded = InArgs._OnSearchBoxExpanded;
		SearchStyle = InArgs._Style;

		SearchBox->SetVisibility(TAttribute<EVisibility>::CreateSP(this, &SExpandableSearchButton::GetSearchBoxVisibility));
		SearchBoxPtr = SearchBox;

		ChildSlot
		[
			SNew(SCheckBox)
			.IsChecked(this, &SExpandableSearchButton::GetToggleButtonState)
			.OnCheckStateChanged(this, &SExpandableSearchButton::OnToggleButtonStateChanged)
			.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
			.Padding(2.0f)
			.ToolTipText(NSLOCTEXT("ExpandableSearchArea", "ExpandCollapseSearchButton", "Expands or collapses the search text box"))
			[
				SNew(SImage)
				.Image(&SearchStyle->GlassImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];
	}

private:
	/** Sets whether or not the search area is expanded to expose the search box */
	void OnToggleButtonStateChanged(ECheckBoxState CheckBoxState)
	{
		bIsExpanded = CheckBoxState == ECheckBoxState::Checked;

		if (TSharedPtr<SSearchBox> SearchBox = SearchBoxPtr.Pin())
		{
			if (bIsExpanded)
			{
				OnSearchBoxExpanded.ExecuteIfBound();

				// Focus the search box when it's shown
				FSlateApplication::Get().SetUserFocus(FSlateApplication::Get().GetUserIndexForKeyboard(), SearchBox, EFocusCause::SetDirectly);
			}
			else
			{
				// Clear the search box when it's hidden
				SearchBox->SetText(FText::GetEmpty());
			}
		}
	}

	ECheckBoxState GetToggleButtonState() const { return bIsExpanded ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; }
	EVisibility GetSearchBoxVisibility() const { return bIsExpanded ? EVisibility::Visible : EVisibility::Collapsed; }

private:
	const FSearchBoxStyle* SearchStyle;
	FOnSearchBoxExpanded OnSearchBoxExpanded;
	TWeakPtr<SSearchBox> SearchBoxPtr;
	bool bIsExpanded = false;
};

/** An expanded area to contain the changelists tree view or then uncontrolled changelists tree view. */
class SExpandableChangelistArea : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SExpandableChangelistArea)
		: _Style(&FAppStyle::Get().GetWidgetStyle<FSearchBoxStyle>("SearchBox"))
		, _HeaderText()
		, _ChangelistView()
		, _OnSearchBoxExpanded()
		, _OnNewChangelist()
		, _OnNewChangelistTooltip()
		, _NewButtonVisibility(EVisibility::Visible)
		, _SearchButtonVisibility(EVisibility::Visible)
		, _OnSearchTextChanged()
	{}
		/** Search box style (used to match the glass icon) */
		SLATE_STYLE_ARGUMENT(FSearchBoxStyle, Style)
		/** Text displayed on the expandable area */
		SLATE_ATTRIBUTE(FText, HeaderText)
		/** The tree element displayed as body. */
		SLATE_ARGUMENT(TSharedPtr<STreeView<FChangelistTreeItemPtr>>, ChangelistView)
		/** Event fired when the associated search box is made visible */
		SLATE_EVENT(FOnSearchBoxExpanded, OnSearchBoxExpanded)
		/** Event fired when the 'plus' button is clicked. */
		SLATE_EVENT(FOnClicked, OnNewChangelist)
		/** Tooltip displayed over the 'plus' button. */
		SLATE_ATTRIBUTE(FText, OnNewChangelistTooltip)
		/** Make the 'plus' button visible or not. */
		SLATE_ARGUMENT(EVisibility, NewButtonVisibility)
		/** Make the 'search' button visible or not. */
		SLATE_ARGUMENT(EVisibility, SearchButtonVisibility)
		/** Invoked whenever the searched text changes. */
		SLATE_EVENT(FOnTextChanged, OnSearchTextChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		SearchBox = SNew(SSearchBox)
			.OnTextChanged(InArgs._OnSearchTextChanged);

		ChildSlot
		[
			SAssignNew(ExpandableArea, SExpandableArea)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
			.BodyBorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
			.HeaderPadding(FMargin(4.0f, 2.0f))
			.AllowAnimatedTransition(false)
			.HeaderContent()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(InArgs._HeaderText)
					.TextStyle(FAppStyle::Get(), "ButtonText")
					.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
				]
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.AutoWidth()
				.Padding(4.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ToolTipText(InArgs._OnNewChangelistTooltip)
					.OnClicked(InArgs._OnNewChangelist)
					.ContentPadding(FMargin(1, 0))
					.Visibility(InArgs._NewButtonVisibility)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.AutoWidth()
				.Padding(4.0f, 0.0f)
				[
					SNew(SExpandableSearchButton, SearchBox.ToSharedRef())
						.Visibility(InArgs._SearchButtonVisibility)
				]
			]
			.BodyContent()
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					// Should blend in visually with the header but technically acts like part of the body
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
					.Padding(FMargin(4.0f, 2.0f))
					[
						SearchBox.ToSharedRef()
					]
				]
				+ SVerticalBox::Slot()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
					[
						InArgs._ChangelistView.ToSharedRef()
					]
				]
			]
		];
	}

	void SetExpanded(bool bExpanded) { ExpandableArea->SetExpanded(bExpanded); }
	bool IsExpanded() const { return ExpandableArea->IsExpanded(); }
	FText GetSearchedText() const { return SearchBox->GetText(); }
	TSharedPtr<SSearchBox> GetSearchBox() { return SearchBox; }

private:
	TSharedPtr<SExpandableArea> ExpandableArea;
	TSharedPtr<SSearchBox> SearchBox;
};


void SSourceControlChangelistsWidget::FAsyncTimestampUpdater::DoWork()
{
	for (FString& Pathname : RequestedFileTimestamps)
	{
		if (!bAbandon)
		{
			if (FPaths::IsRelative(Pathname))
			{
				Pathname = FPaths::ConvertRelativePathToFull(Pathname);
			}
			QueriedFileTimestamps.Emplace(MoveTemp(Pathname), IFileManager::Get().GetTimeStamp(*Pathname));
		}
	}
}


void SSourceControlChangelistsWidget::Construct(const FArguments& InArgs)
{
	bShowingContentVersePath = FAssetToolsModule::GetModule().Get().ShowingContentVersePath();

	// Register delegates
	ISourceControlModule& SCCModule = ISourceControlModule::Get();
	FUncontrolledChangelistsModule& UncontrolledChangelistModule = FUncontrolledChangelistsModule::Get();

	SCCModule.RegisterProviderChanged(FSourceControlProviderChanged::FDelegate::CreateSP(this, &SSourceControlChangelistsWidget::OnSourceControlProviderChanged));
	SourceControlStateChangedDelegateHandle = SCCModule.GetProvider().RegisterSourceControlStateChanged_Handle(FSourceControlStateChanged::FDelegate::CreateSP(this, &SSourceControlChangelistsWidget::OnSourceControlStateChanged));
	UncontrolledChangelistModule.OnUncontrolledChangelistModuleChanged.AddSP(this, &SSourceControlChangelistsWidget::OnUncontrolledChangelistStateChanged);

	// Monitor when packages are saved to update the timestamps.
	UPackage::PackageSavedWithContextEvent.AddSP(this, &SSourceControlChangelistsWidget::OnPackageSaved);

	IPluginManager::Get().OnPluginEdited().AddSP(this, &SSourceControlChangelistsWidget::OnPluginEdited);

	// No default sorting, sorting make the view slower, just pay the cost if user wants it.
	PrimarySortedColumn = FName();

	UnsavedAssetsTreeNode.Add(MakeShared<FUnsavedAssetsTreeItem>());
	
	UnsavedAssetsTreeView = CreateChangelistTreeView(UnsavedAssetsTreeNode);
	ChangelistTreeView = CreateChangelistTreeView(ChangelistTreeNodes);
	UncontrolledChangelistTreeView = CreateChangelistTreeView(UncontrolledChangelistTreeNodes);
	FileListView = CreateChangelistFilesView();
	UnsavedAssetsFileListView = CreateUnsavedAssetsFilesView();
	
	FileListViewSwitcher = SNew(SWidgetSwitcher);
	FileListViewSwitcher->AddSlot().AttachWidget(FileListView.ToSharedRef());
	FileListViewSwitcher->AddSlot().AttachWidget(UnsavedAssetsFileListView.ToSharedRef());

	UnsavedAssetsExpandableArea = SNew(SExpandableChangelistArea)
		.HeaderText_Lambda([] { return FText::Format(LOCTEXT("SourceControl_UnsavedAssets", "Unsaved Assets ({0})"), FUnsavedAssetsTrackerModule::Get().GetUnsavedAssetNum()); })
		.ChangelistView(UnsavedAssetsTreeView.ToSharedRef())
		.NewButtonVisibility(EVisibility::Collapsed)
		.SearchButtonVisibility(EVisibility::Collapsed);
	
	ChangelistExpandableArea = SNew(SExpandableChangelistArea)
		.HeaderText_Lambda([this]() { return FText::Format(LOCTEXT("SourceControl_ChangeLists", "Changelists ({0})"), ChangelistTreeNodes.Num()); })
		.ChangelistView(ChangelistTreeView.ToSharedRef())
		.OnNewChangelist_Lambda([this](){ OnNewChangelist(); return FReply::Handled(); })
		.OnNewChangelistTooltip(LOCTEXT("Create_New_Changelist", "Create a new changelist."))
		.SearchButtonVisibility(EVisibility::Visible)
		.OnSearchTextChanged(this, &SSourceControlChangelistsWidget::OnChangelistSearchTextChanged);

	UncontrolledChangelistExpandableArea = SNew(SExpandableChangelistArea)
		.HeaderText_Lambda([this]() { return FText::Format(LOCTEXT("SourceControl_UncontrolledChangeLists", "Uncontrolled Changelists ({0})"), UncontrolledChangelistTreeNodes.Num()); })
		.ChangelistView(UncontrolledChangelistTreeView.ToSharedRef())
		.OnNewChangelist_Lambda([this]() { OnNewUncontrolledChangelist(); return FReply::Handled(); })
		.OnNewChangelistTooltip(LOCTEXT("Create_New_Uncontrolled_Changelist", "Create a new uncontrolled changelist."))
		.SearchButtonVisibility(EVisibility::Visible) // Functionality is planned but not fully implemented yet.
		.OnSearchTextChanged(this, &SSourceControlChangelistsWidget::OnUncontrolledChangelistSearchTextChanged);

	ChangelistTextFilter = MakeShared<TTextFilter<const IChangelistTreeItem&>>(TTextFilter<const IChangelistTreeItem&>::FItemToStringArray::CreateSP(this, &SSourceControlChangelistsWidget::PopulateItemSearchStrings));
	ChangelistTextFilter->OnChanged().AddSP(this, &SSourceControlChangelistsWidget::OnRefreshUI, ERefreshFlags::SourceControlChangelists);

	UncontrolledChangelistTextFilter = MakeShared<TTextFilter<const IChangelistTreeItem&>>(TTextFilter<const IChangelistTreeItem&>::FItemToStringArray::CreateSP(this, &SSourceControlChangelistsWidget::PopulateItemSearchStrings));
	UncontrolledChangelistTextFilter->OnChanged().AddSP(this, &SSourceControlChangelistsWidget::OnRefreshUI, ERefreshFlags::UncontrolledChangelists);

	FileTextFilter = MakeShared<TTextFilter<const IChangelistTreeItem&>>(TTextFilter<const IChangelistTreeItem&>::FItemToStringArray::CreateSP(this, &SSourceControlChangelistsWidget::PopulateItemSearchStrings));
	FileTextFilter->OnChanged().AddSP(this, &SSourceControlChangelistsWidget::OnRefreshUI, ERefreshFlags::All);

	FUnsavedAssetsTrackerModule::Get().OnUnsavedAssetAdded.AddSP(this, &SSourceControlChangelistsWidget::OnUnsavedAssetChanged);
	FUnsavedAssetsTrackerModule::Get().OnUnsavedAssetRemoved.AddSP(this, &SSourceControlChangelistsWidget::OnUnsavedAssetChanged);

	// Min/Max prevents making the Changelist Area too small and consequently prevent the file search box to go over the 'refresh' button.
	const float MinChangelistAreaRatio = 0.2f;
	const float MaxFileAreaRation = 1.0f - MinChangelistAreaRatio;

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot() // For the toolbar (Refresh button)
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(4)
			[
				SNew(SOverlay) // To align the file search box with the Changelist/File splitter, to compute ratio on the same width.
				+SOverlay::Slot()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						MakeToolBar()
					]
				]

				+SOverlay::Slot() // That slots has the same width than the SSplitter and use the same left/right ratio to keep the search box over the file section.
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.FillWidth(TAttribute<float>::CreateLambda([this, MinChangelistAreaRatio]() { return FMath::Max(MinChangelistAreaRatio, ChangelistAreaSize / (ChangelistAreaSize + FileAreaSize)); }))
					.VAlign(VAlign_Center)
					[
						SNew(SSpacer) // This spacer uses the same width as the Changelist area on the left. They are kept in sync.
					]

					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.FillWidth(TAttribute<float>::CreateLambda([this, MaxFileAreaRation]() { return FMath::Min(MaxFileAreaRation, FileAreaSize / (ChangelistAreaSize + FileAreaSize)); }))
					[
						SAssignNew(FileSearchBox, SSearchBox)
						.HintText(LOCTEXT("Search_Files", "Search the files..."))
						.OnTextChanged(this, &SSourceControlChangelistsWidget::OnFileSearchTextChanged)
					]
				]
			]
		]
		+SVerticalBox::Slot() // Everything below the tools bar: changelist expandable areas + files views + status bar at the bottom
		[
			SNew(SSplitter)
			.Orientation(EOrientation::Orient_Horizontal)
			.ResizeMode(ESplitterResizeMode::FixedPosition)

			// Left slot: Changelists and uncontrolled changelists areas
			+SSplitter::Slot()
			.Resizable(true)
			.SizeRule(SSplitter::FractionOfParent)
			.Value_Lambda([this, MinChangelistAreaRatio]() { return FMath::Max(MinChangelistAreaRatio, ChangelistAreaSize); })
			.OnSlotResized_Lambda([this](float Size) { ChangelistAreaSize = Size; })
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBox)
					.Visibility_Lambda([] { return AreUnsavedAssetsPresent() ? EVisibility::Visible : EVisibility::Collapsed; } )
					[
						UnsavedAssetsExpandableArea.ToSharedRef()
					]
				]
				+SVerticalBox::Slot()
				[
					SNew(SOverlay)
					+SOverlay::Slot()
					[
						SNew(SBox)
						.Visibility_Lambda([](){ return !AreChangelistsEnabled() ? EVisibility::Visible: EVisibility::Collapsed; })
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("SourceControl_Disabled", "The revision control is disabled or it doesn't support changelists."))
						]
					]
					+SOverlay::Slot() // Visible when both Controlled and Uncontrolled changelists are enabled (Need to split vertical space)
					[
						SNew(SSplitter)
						.Orientation(EOrientation::Orient_Vertical)
						.Visibility_Lambda([]() { return AreControlledChangelistsEnabled() && AreUncontrolledChangelistsEnabled() ? EVisibility::Visible : EVisibility::Collapsed; })
						
						// Top slot: Changelists
						+SSplitter::Slot()
						.SizeRule_Lambda([this](){ return ChangelistExpandableArea->IsExpanded() ? SSplitter::ESizeRule::FractionOfParent : SSplitter::ESizeRule::SizeToContent; })
						.Value(0.7)
						[
							ChangelistExpandableArea.ToSharedRef()
						]
						
						// Bottom slot: Uncontrolled Changelists
						+SSplitter::Slot()
						.SizeRule_Lambda([this](){ return UncontrolledChangelistExpandableArea->IsExpanded() ? SSplitter::ESizeRule::FractionOfParent : SSplitter::ESizeRule::SizeToContent; })
						.Value(0.3)
						[
							UncontrolledChangelistExpandableArea.ToSharedRef()
						]
					]
					+SOverlay::Slot() // Visible when controlled changelists are enabled but not the uncontrolled ones.
					[
						SNew(SBox)
						.Visibility_Lambda([](){ return AreControlledChangelistsEnabled() && !AreUncontrolledChangelistsEnabled() ? EVisibility::Visible : EVisibility::Collapsed; })
						[
							ChangelistExpandableArea.ToSharedRef()
						]
					]
					+SOverlay::Slot() // Visible when uncontrolled changelist are enabled, but not the controlled ones.
					[
						SNew(SBox)
						.Visibility_Lambda([](){ return !AreControlledChangelistsEnabled() && AreUncontrolledChangelistsEnabled() ? EVisibility::Visible : EVisibility::Collapsed; })
						[
							UncontrolledChangelistExpandableArea.ToSharedRef()
						]
					]
				]
			]

			// Right slot: Files associated to the selected the changelist/uncontrolled changelist.
			+SSplitter::Slot()
			.Resizable(true)
			.SizeRule(SSplitter::FractionOfParent)
			.Value_Lambda([this, MaxFileAreaRation]() { return FMath::Min(MaxFileAreaRation, FileAreaSize); })
			.OnSlotResized_Lambda([this](float Size) { FileAreaSize = Size; })
			[
				FileListViewSwitcher.ToSharedRef()
			]
		]
		+SVerticalBox::Slot() // Status bar (Always visible if uncontrolled changelist are enabled to keep the reconcile status visible at all time)
		.AutoHeight()
		[
			SNew(SBox)
			.Padding(0, 3)
			.Visibility_Lambda([this](){ return FUncontrolledChangelistsModule::Get().IsEnabled() || !RefreshStatus.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed; })
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text_Lambda([this]() { return RefreshStatus; })
				]
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text_Lambda([]() { return FUncontrolledChangelistsModule::Get().GetReconcileStatus(); })
					.Visibility_Lambda([]() { return AreUncontrolledChangelistsEnabled() ? EVisibility::Visible : EVisibility::Collapsed; })
				]
			]
		]
	];

	bShouldRefresh = true;
}

SSourceControlChangelistsWidget::~SSourceControlChangelistsWidget()
{
	ISourceControlModule::Get().GetSourceControlFileStatusMonitor().StopMonitoringFiles(reinterpret_cast<uintptr_t>(this));

	if (TimestampUpdateTask)
	{
		TimestampUpdateTask->TryAbandonTask();
		TimestampUpdateTask->EnsureCompletion(/*bDoWorkOnThisThreadIfNotStarted*/false);
		TimestampUpdateTask.Reset();
	}
}

TSharedRef<SWidget> SSourceControlChangelistsWidget::MakeToolBar()
{
	FSlimHorizontalToolBarBuilder ToolBarBuilder(nullptr, FMultiBoxCustomization::None);

	ToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateLambda([this]() { RequestChangelistsRefresh(); })),
			NAME_None,
			LOCTEXT("SourceControl_RefreshButton", "Refresh"),
			LOCTEXT("SourceControl_RefreshButton_Tooltip", "Refreshes changelists from revision control provider."),
			FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.Refresh"));

	return ToolBarBuilder.MakeWidget();
}

void SSourceControlChangelistsWidget::EditChangelistDescription(const FText& InNewChangelistDescription, const FSourceControlChangelistStatePtr& InChangelistState)
{
	if (InChangelistState->SupportsPersistentDescription())
	{
		TSharedRef<FEditChangelist> EditChangelistOperation = ISourceControlOperation::Create<FEditChangelist>();
		EditChangelistOperation->SetDescription(InNewChangelistDescription);
		Execute(LOCTEXT("Updating_Changelist_Description", "Updating changelist description..."), EditChangelistOperation, InChangelistState->GetChangelist(), EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateLambda(
			[](const TSharedRef<ISourceControlOperation>& Operation, ECommandResult::Type InResult)
			{
				if (InResult == ECommandResult::Succeeded)
				{
					SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Update_Changelist_Description_Succeeded", "Changelist description successfully updated."), SNotificationItem::CS_Success);
				}
				else if (InResult == ECommandResult::Failed)
				{
					SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Update_Changelist_Description_Failed", "Failed to update changelist description."), SNotificationItem::CS_Fail);
				}
			}));
	}
	else // Move everything to a new CL. Ex: the default P4 CL description cannot be saved.
	{
		TArray<FString> FilesToMove;
		Algo::Transform(InChangelistState->GetFilesStates(), FilesToMove, [](const TSharedRef<ISourceControlState>& FileState) { return FileState->GetFilename(); });

		TSharedRef<FNewChangelist> NewChangelistOperation = ISourceControlOperation::Create<FNewChangelist>();
		NewChangelistOperation->SetDescription(InNewChangelistDescription);
		Execute(LOCTEXT("Saving_Into_New_Changelist", "Saving to a new changelist..."), NewChangelistOperation, FilesToMove, EConcurrency::Synchronous, FSourceControlOperationComplete::CreateLambda(
			[](const TSharedRef<ISourceControlOperation>& Operation, ECommandResult::Type InResult)
			{
				if (InResult == ECommandResult::Succeeded)
				{
					SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Saving_New_Changelist_Succeeded", "New changelist saved"), SNotificationItem::CS_Success);
				}
				if (InResult == ECommandResult::Failed)
				{
					SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Saving_New_Changelist_Failed", "Failed to save to a new changelist."), SNotificationItem::CS_Fail);
				}
			}));
	}
}

void SSourceControlChangelistsWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (!ISourceControlModule::Get().IsEnabled() && !FUncontrolledChangelistsModule::Get().IsEnabled())
	{
		return;
	}

	// Detect transitions of the source control being available/unavailable. Ex: When the user changes the source control in UI, the provider gets selected,
	// but it is not connected/available until the user accepts the settings. The source control doesn't have callback for availability and we want to refresh everything
	// once it gets available.
	if (ISourceControlModule::Get().IsEnabled() && !bSourceControlAvailable && ISourceControlModule::Get().GetProvider().IsAvailable())
	{
		bSourceControlAvailable = true;
		bShouldRefresh = true;
	}

	bool bSortFileView = false;

	if (bShouldRefresh)
	{
		if (!bInitialRefreshDone)
		{
			// Ensure the UI is built from the current cached states once because all UI updates are hooked on 'state change' callbacks and those will only
			// be called when the state changed.
			OnRefreshUI(ERefreshFlags::All);
			bInitialRefreshDone = true;
		}

		RequestChangelistsRefresh();
		bShouldRefresh = false;
	}

	if (bIsRefreshing)
	{
		TickRefreshStatus(InDeltaTime);
	}

	if (bUpdateMonitoredFileStatusList)
	{
		if (FileListNodes.IsEmpty())
		{
			// Nothing to monitor.
			ISourceControlModule::Get().GetSourceControlFileStatusMonitor().StopMonitoringFiles(reinterpret_cast<uintptr_t>(this));
			bUpdateMonitoredFileStatusList = false;
		}
		else if (IsFileViewSortedByFileStatusIcon())
		{
			// If a changelist is selected, monitor all files status to honor the sort order on that column.
			if (ChangelistTreeView->GetNumItemsSelected() > 0)
			{
				RequestFileStatusRefresh(*ChangelistTreeView->GetSelectedItems()[0]);
			}
			else if (UncontrolledChangelistTreeView->GetNumItemsSelected() > 0)
			{
				RequestFileStatusRefresh(*UncontrolledChangelistTreeView->GetSelectedItems()[0]);
			}
			else if (UnsavedAssetsTreeView->GetNumItemsSelected() > 0)
			{
				RequestFileStatusRefresh(*UnsavedAssetsTreeView->GetSelectedItems()[0]);
			}
			bUpdateMonitoredFileStatusList = false;
		}
		else // Just monitor the files that are visible in the view.
		{
			// NOTE: In the tick where the view is updated/refreshed, the item count is usually zero, next tick it will have the correct value, so wait for it.
			int32 ItemCount = GetActiveFileListView().GetNumGeneratedChildren();
			if (ItemCount > 0)
			{
				// Minimize the list of file status monitored to only include those in view. The ones outside of the visible area don't need
				// to be monitored unless the user sorts.
				int32 ItemStart = static_cast<int32>(GetActiveFileListView().GetScrollOffset()); // This give the offset in item count.
				int32 ItemEnd = ItemStart + ItemCount;
				TSet<FString> MonitoredFiles;
				for (int32 Index = ItemStart; Index < ItemEnd; ++Index)
				{
					MonitoredFiles.Emplace(StaticCastSharedPtr<IFileViewTreeItem>(FileListNodes[Index])->GetFullPathname());
				}
				RequestFileStatusRefresh(MoveTemp(MonitoredFiles));
				bUpdateMonitoredFileStatusList = false;
			}
		}
	}

	{
		const bool bNewShowingContentVersePath = FAssetToolsModule::GetModule().Get().ShowingContentVersePath();
		if (bShowingContentVersePath != bNewShowingContentVersePath)
		{
			bShowingContentVersePath = bNewShowingContentVersePath;
			bSortFileView = true;
		}
	}

	if (bShouldRefreshVersePaths && bShowingContentVersePath)
	{
		bShouldRefreshVersePaths = false;

		for (const TSharedPtr<IChangelistTreeItem>& Item : FileListNodes)
		{
			// Resort of any of the Verse paths change.
			bSortFileView |= static_cast<IFileViewTreeItem*>(Item.Get())->RefreshVersePath();
		}
	}

	// If requested timestamps were gathered in background.
	if (TimestampUpdateTask && TimestampUpdateTask->IsDone())
	{
		bool bUpdatedItemInView = false;

		// Start updating the offline files (if any) as we have a direct mapping between keys.
		if (OfflineFileItemCache.Num() > 0)
		{
			for (auto It = TimestampUpdateTask->GetTask().QueriedFileTimestamps.CreateIterator(); It; ++It)
			{
				if (TSharedPtr<IFileViewTreeItem>* FileViewItem = OfflineFileItemCache.Find(It->Key); FileViewItem && FileViewItem->IsValid())
				{
					(*FileViewItem)->SetLastModifiedDateTime(It->Value);
					bUpdatedItemInView |= (*FileViewItem)->DisplayedUpdateNum == UpdateRequestNum;
					It.RemoveCurrent();
				}
			}
		}

		// Update other files. There is no direct lookup key File -> FileItem, do a linear scan of the cache.
		int32 UpdateRemaining = TimestampUpdateTask->GetTask().QueriedFileTimestamps.Num();
		auto UpdateCachedItems = [this, &UpdateRemaining, &bUpdatedItemInView](TMap<TSharedPtr<void>, TSharedPtr<IChangelistTreeItem>>& Cache)
		{
			for (auto It = Cache.CreateIterator(); It && UpdateRemaining > 0; ++It)
			{
				if (It->Value->GetTreeItemType() == IChangelistTreeItem::File || It->Value->GetTreeItemType() == IChangelistTreeItem::ShelvedFile)
				{
					IFileViewTreeItem* FileViewItem = static_cast<IFileViewTreeItem*>(It->Value.Get());
					if (const FDateTime* Timestamp = TimestampUpdateTask->GetTask().QueriedFileTimestamps.Find(FileViewItem->GetFullPathname()))
					{
						FileViewItem->SetLastModifiedDateTime(*Timestamp);
						bUpdatedItemInView |= FileViewItem->DisplayedUpdateNum == UpdateRequestNum;
						--UpdateRemaining;
					}
				}
			}
		};
		UpdateCachedItems(SourceControlItemCache);
		UpdateCachedItems(UncontrolledChangelistItemCache);
		TimestampUpdateTask.Reset();

		if (bUpdatedItemInView && IsFileViewSortedByLastModifiedTimestamp())
		{
			bSortFileView = true;
		}
	}

	if (bSortFileView)
	{
		SortFileView();
		GetActiveFileListView().RequestListRefresh();
	}

	if (!TimestampUpdateTask && OutdatedTimestampFiles.Num()) // Should be sufficient to run only one async task at the time.
	{
		// Query the timestamps asynchronously.
		TimestampUpdateTask = MakeUnique<FAsyncTask<FAsyncTimestampUpdater>>();
		TimestampUpdateTask->GetTask().RequestedFileTimestamps = MoveTemp(OutdatedTimestampFiles);
		TimestampUpdateTask->StartBackgroundTask();
	}
}

void SSourceControlChangelistsWidget::RequestChangelistsRefresh()
{
	bool bAnyProviderAvailable = false;

	if (AreControlledChangelistsEnabled())
	{
		bAnyProviderAvailable = true;
		StartRefreshStatus();

		TSharedRef<FUpdatePendingChangelistsStatus, ESPMode::ThreadSafe> UpdatePendingChangelistsOperation = ISourceControlOperation::Create<FUpdatePendingChangelistsStatus>();
		UpdatePendingChangelistsOperation->SetUpdateAllChangelists(true);
		UpdatePendingChangelistsOperation->SetUpdateFilesStates(true);
		UpdatePendingChangelistsOperation->SetUpdateShelvedFilesStates(true);

		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		SourceControlProvider.Execute(UpdatePendingChangelistsOperation, EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &SSourceControlChangelistsWidget::OnChangelistsStatusUpdated));
		OnStartSourceControlOperation(UpdatePendingChangelistsOperation, LOCTEXT("SourceControl_UpdatingChangelist", "Updating changelists..."));
	}

	if (AreUncontrolledChangelistsEnabled())
	{
		bAnyProviderAvailable = true;

		// This operation is synchronous and completes right away.
		FUncontrolledChangelistsModule::Get().UpdateStatus();
	}

	if (!bAnyProviderAvailable)
	{
		// No provider available, clear changelist tree
		ClearChangelistsTree();
	}
}

void SSourceControlChangelistsWidget::RequestFileStatusRefresh(const IChangelistTreeItem& Changelist)
{
	TSet<FString> Pathnames;
	if (Changelist.GetTreeItemType() == IChangelistTreeItem::Changelist)
	{
		const FChangelistTreeItem& ChangelistItem = static_cast<const FChangelistTreeItem&>(Changelist);
		Algo::Transform(ChangelistItem.ChangelistState->GetFilesStates(), Pathnames, [](const TSharedRef<ISourceControlState>& FileState) { return FileState->GetFilename(); });
	}
	else if (Changelist.GetTreeItemType() == IChangelistTreeItem::ShelvedChangelist)
	{
		const FChangelistTreeItem& ChangelistItem = static_cast<const FChangelistTreeItem&>(*Changelist.GetParent().Get());
		Algo::Transform(ChangelistItem.ChangelistState->GetShelvedFilesStates(), Pathnames, [](const TSharedRef<ISourceControlState>& FileState) { return FileState->GetFilename(); });
	}
	else if (Changelist.GetTreeItemType() == IChangelistTreeItem::UncontrolledChangelist)
	{
		const FUncontrolledChangelistTreeItem& ChangelistItem = static_cast<const FUncontrolledChangelistTreeItem&>(Changelist);
		Algo::Transform(ChangelistItem.UncontrolledChangelistState->GetFilenames(), Pathnames, [](const FString& Pathname) { return Pathname; });
	}

	if (!Pathnames.IsEmpty())
	{
		// Fire an async task to get the latest files status from source control to get the 'Checked Out By' status.
		RequestFileStatusRefresh(MoveTemp(Pathnames));
	}
}

void SSourceControlChangelistsWidget::RequestFileStatusRefresh(TSet<FString>&& PathnamesToMonitor)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SSourceControlChangelistsWidget::RequestFileStatusRefresh);

	if (!ISourceControlModule::Get().IsEnabled())
	{
		return;
	}

	// NOTE: This is using the file status monitor and the status might be a bit old but not older than the
	//       update period policy of the monitor. See FSourceControlFileStatusMonitor::SetUpdateStatusPeriodPolicy()

	// If the changelist contains files.
	if (!PathnamesToMonitor.IsEmpty())
	{
		// Request an update only for the files that changed.
		ISourceControlModule::Get().GetSourceControlFileStatusMonitor().SetMonitoringFiles(reinterpret_cast<uintptr_t>(this), MoveTemp(PathnamesToMonitor), FSourceControlFileStatusMonitor::FOnSourceControlFileStatus());
		// NOTE: The updated status are expected to come from OnSourceControlStateChanged() that is invoked when the source control internal state changes.
	}
	else
	{
		ISourceControlModule::Get().GetSourceControlFileStatusMonitor().StopMonitoringFiles(reinterpret_cast<uintptr_t>(this));
	}
}

void SSourceControlChangelistsWidget::StartRefreshStatus()
{
	bIsRefreshing = true;
	RefreshStatusStartSecs = FPlatformTime::Seconds();
}

void SSourceControlChangelistsWidget::TickRefreshStatus(double InDeltaTime)
{
	int32 RefreshStatusTimeElapsed = static_cast<int32>(FPlatformTime::Seconds() - RefreshStatusStartSecs);
	RefreshStatus = FText::Format(LOCTEXT("SourceControl_RefreshStatus", "Refreshing changelists... ({0} s)"), FText::AsNumber(RefreshStatusTimeElapsed));
}

void SSourceControlChangelistsWidget::EndRefreshStatus()
{
	bIsRefreshing = false;
}

void SSourceControlChangelistsWidget::ClearChangelistsTree()
{
	if (!ChangelistTreeNodes.IsEmpty())
	{
		ChangelistTreeNodes.Reset();
		ChangelistTreeView->RequestTreeRefresh();
	}

	if (!UncontrolledChangelistTreeNodes.IsEmpty())
	{
		UncontrolledChangelistTreeNodes.Reset();
		UncontrolledChangelistTreeView->RequestTreeRefresh();
	}

	SourceControlItemCache.Reset();
	UncontrolledChangelistItemCache.Reset();
	OfflineFileItemCache.Reset();
}

void SSourceControlChangelistsWidget::OnRefreshUnsavedAssetsWidgets(int64 CurrUpdateNum, const TFunction<void(TSharedPtr<IFileViewTreeItem>&)>& AddItemToFileView)
{
	const bool bBeautifyPaths = true;
	const int64 PrevUpdateNum = CurrUpdateNum - 1;
	int32 DisplayedItemCount = 0;
	int32 VisitedItemCount = 0;

	TSharedPtr<IChangelistTreeItem> UnsavedAssetsItem = UnsavedAssetsTreeView->GetNumItemsSelected() > 0 ? UnsavedAssetsTreeView->GetSelectedItems()[0] : TSharedPtr<IChangelistTreeItem>();

	if (!UnsavedAssetsItem)
	{
		return;
	}

	auto OnSourceControlCachedItemVisited = [CurrUpdateNum, &VisitedItemCount](IChangelistTreeItem* Item)
	{
		checkf(Item->VisitedUpdateNum < CurrUpdateNum, TEXT("The same revision control item was visited twice. It is likely present in more than one changelist"));
		Item->VisitedUpdateNum = CurrUpdateNum;
		++VisitedItemCount;
	};

	auto OnNewSourceControlItemCreated = [this, &OnSourceControlCachedItemVisited](FString Key, TSharedPtr<IFileViewTreeItem> Item)
	{
		OnSourceControlCachedItemVisited(Item.Get());
		OfflineFileItemCache.Emplace(MoveTemp(Key), MoveTemp(Item));
	};

	auto AddItemToFileViewWrapper = [&AddItemToFileView, &DisplayedItemCount](TSharedPtr<IFileViewTreeItem>& Item)
	{
		++DisplayedItemCount;
		AddItemToFileView(Item);
	};

	auto RemoveDiscardedSourceControlItems = [this, VisitedItemCount, DisplayedItemCount, CurrUpdateNum]()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SSourceControlChangelistsWidget::RemoveDiscardedSourceControlItems);

		check(OfflineFileItemCache.Num() >= VisitedItemCount); // Cannot visit items that are not cached.

		// Remove the items used to be visited but weren't visited during this iteration.
		int32 DiscardCount = OfflineFileItemCache.Num() - VisitedItemCount;
		for (auto It = OfflineFileItemCache.CreateIterator(); It && DiscardCount > 0 ; ++It)
		{
			if (It->Value->VisitedUpdateNum < CurrUpdateNum)
			{
				It.RemoveCurrent();
				--DiscardCount;
			}
		}
		
		// Unless the FileTreeNode array was reset because the selected changelist changed, it still contains the union of items to display for this iteration and the item that were displayed last
		// iteration, so we need to remove item that aren't displayed anymore. (The FileTreeNode is not reset between updates to preserve the sort order as much as possible).
		if (FileListNodes.Num() > DisplayedItemCount)
		{
			// Remove items that were not displayed anymore.
			FileListNodes.RemoveAll([CurrUpdateNum](const TSharedPtr<IChangelistTreeItem>& Candidate) { return Candidate->DisplayedUpdateNum < CurrUpdateNum; });
		}
	};

	for (const FString& UnsavedAssetPath : FUnsavedAssetsTrackerModule::Get().GetUnsavedAssets())
	{
		TSharedPtr<IFileViewTreeItem> FileItem;
		if (TSharedPtr<IFileViewTreeItem>* CachedFileItem = OfflineFileItemCache.Find(UnsavedAssetPath); CachedFileItem && CachedFileItem->IsValid())
		{
			check((*CachedFileItem)->GetTreeItemType() == IChangelistTreeItem::OfflineFile);
			FileItem = StaticCastSharedPtr<FOfflineFileTreeItem>(*CachedFileItem);
			OnSourceControlCachedItemVisited(FileItem.Get());
		}
		else
		{
			FileItem = MakeShared<FOfflineFileTreeItem>(UnsavedAssetPath);
			OnNewSourceControlItemCreated(UnsavedAssetPath, FileItem);
		}

		if (FileTextFilter->PassesFilter(*FileItem))
		{
			UnsavedAssetsItem->AddChild(FileItem.ToSharedRef());
			AddItemToFileViewWrapper(FileItem);
		}
	}

	RemoveDiscardedSourceControlItems();
	UnsavedAssetsTreeView->RequestTreeRefresh();
}

void SSourceControlChangelistsWidget::OnRefreshSourceControlWidgets(int64 CurrUpdateNum, const TFunction<void(TSharedPtr<IFileViewTreeItem>&)>& AddItemToFileView)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SSourceControlChangelistsWidget::OnRefreshSourceControlWidgets);

	const bool bBeautifyPaths = true;
	const int64 PrevUpdateNum = CurrUpdateNum - 1;
	int32 DisplayedItemCount = 0;
	int32 VisitedItemCount = 0;
	TArray<IChangelistTreeItem*> DuplicateItems;

	TSharedPtr<IChangelistTreeItem> SelectedChangelistItem = ChangelistTreeView->GetNumItemsSelected() > 0 ? ChangelistTreeView->GetSelectedItems()[0] : TSharedPtr<IChangelistTreeItem>();

	auto HasSourceControlChangelistSelected = [&SelectedChangelistItem]()
	{
		return SelectedChangelistItem && 
			(SelectedChangelistItem->GetTreeItemType() == IChangelistTreeItem::Changelist || SelectedChangelistItem->GetTreeItemType() == IChangelistTreeItem::ShelvedChangelist);
	};

	auto OnSourceControlCachedItemVisited = [CurrUpdateNum, &VisitedItemCount, &DuplicateItems](IChangelistTreeItem* Item)
	{
		if (Item->VisitedUpdateNum < CurrUpdateNum)
		{
			Item->VisitedUpdateNum = CurrUpdateNum;
			++VisitedItemCount;
		}
		else
		{
			// The same revision control item was visited twice. It is likely present in more than one changelist
			// This can happen in the case of a submit from outside of the editor ( e.g. from P4V )
			// Handle it gracefully by ignoring the duplicate and then request a full changelists refresh for next tick
			DuplicateItems.Emplace(Item);
		}
	};

	auto OnNewSourceControlItemCreated = [this, &OnSourceControlCachedItemVisited](TSharedPtr<void> Key, TSharedPtr<IChangelistTreeItem> Item)
	{
		OnSourceControlCachedItemVisited(Item.Get());
		SourceControlItemCache.Emplace(MoveTemp(Key), MoveTemp(Item));
	};

	auto AddItemToFileViewWrapper = [&AddItemToFileView, &DisplayedItemCount](TSharedPtr<IFileViewTreeItem>& Item)
	{
		++DisplayedItemCount;
		AddItemToFileView(Item);
	};

	auto RemoveDiscardedSourceControlItems = [this, VisitedItemCount, DisplayedItemCount, CurrUpdateNum, HasSourceControlChangelistSelected]()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SSourceControlChangelistsWidget::RemoveDiscardedSourceControlItems);

		check(SourceControlItemCache.Num() >= VisitedItemCount); // Cannot visit items that are not cached.

		// Remove the items used to be visited but weren't visited during this iteration.
		int32 DiscardCount = SourceControlItemCache.Num() - VisitedItemCount;
		for (auto It = SourceControlItemCache.CreateIterator(); It && DiscardCount > 0 ; ++It)
		{
			if (It->Value->VisitedUpdateNum < CurrUpdateNum)
			{
				It.RemoveCurrent();
				--DiscardCount;
			}
		}

		// If the source control feeds the file view.
		if (HasSourceControlChangelistSelected())
		{
			// Unless the FileTreeNode array was reset because the selected changelist changed, it still contains the union of items to display for this iteration and the item that were displayed last
			// iteration, so we need to remove item that aren't displayed anymore. (The FileTreeNode is not reset between updates to preserve the sort order as much as possible).
			if (FileListNodes.Num() > DisplayedItemCount)
			{
				// Remove items that were not displayed anymore.
				FileListNodes.RemoveAll([CurrUpdateNum](const TSharedPtr<IChangelistTreeItem>& Candidate) { return Candidate->DisplayedUpdateNum < CurrUpdateNum; });
			}
		}
	};

	// Query the source control
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	TArray<FSourceControlChangelistRef> Changelists = SourceControlProvider.GetChangelists(EStateCacheUsage::Use);
	TArray<FSourceControlChangelistStateRef> ChangelistsStates;
	SourceControlProvider.GetState(Changelists, ChangelistsStates, EStateCacheUsage::Use);

	ChangelistTreeNodes.Reset(ChangelistsStates.Num());

	for (const TSharedRef<ISourceControlChangelistState>& ChangelistState : ChangelistsStates)
	{
		// Create or reuse the 'Changelists' and 'Shelved Files' node.
		TSharedPtr<FChangelistTreeItem> ChangelistTreeItem;
		TSharedPtr<FShelvedChangelistTreeItem> ShelvedChangelistTreeItem;
		if (TSharedPtr<IChangelistTreeItem>* CachedChangelistItem = SourceControlItemCache.Find(ChangelistState); CachedChangelistItem && CachedChangelistItem->IsValid())
		{
			check((*CachedChangelistItem)->GetTreeItemType() == IChangelistTreeItem::Changelist);
			ChangelistTreeItem = StaticCastSharedPtr<FChangelistTreeItem>(*CachedChangelistItem);
			ChangelistTreeItem->RemoveAllChildren(); // Will relink the children against the active filter below.
			ChangelistTreeItem->ShelvedChangelistItem->RemoveAllChildren(); // Will relink the children against the active filter below.
			OnSourceControlCachedItemVisited(ChangelistTreeItem.Get());
		}
		else
		{
			ChangelistTreeItem = MakeShared<FChangelistTreeItem>(ChangelistState);
			ChangelistTreeItem->ShelvedChangelistItem = MakeShared<FShelvedChangelistTreeItem>();
			OnNewSourceControlItemCreated(ChangelistState, ChangelistTreeItem);
		}
		ShelvedChangelistTreeItem = ChangelistTreeItem->ShelvedChangelistItem;

		const bool bShelvedChangelistPassesFilter = ChangelistState->GetShelvedFilesStatesNum() > 0 && ChangelistTextFilter->PassesFilter(*ShelvedChangelistTreeItem);
		const bool bShelvedChangelistMustBeDisplayed = bShelvedChangelistPassesFilter && !ChangelistTextFilter->GetRawFilterText().IsEmpty();
		const bool bChangelistPassedFilter = bShelvedChangelistMustBeDisplayed || ChangelistTextFilter->PassesFilter(*ChangelistTreeItem);
		if (bChangelistPassedFilter)
		{
			ChangelistTreeNodes.Add(ChangelistTreeItem);
			if (bShelvedChangelistPassesFilter)
			{
				ChangelistTreeItem->AddChild(ShelvedChangelistTreeItem.ToSharedRef());
				if (bShelvedChangelistMustBeDisplayed)
				{
					ChangelistTreeView->SetItemExpansion(ChangelistTreeItem, /*bShouldExpand*/true);
				}
			}
		}

		// Create or reuse the 'File' nodes.
		const bool bChangelistSelected = SelectedChangelistItem == ChangelistTreeItem;
		for (const TSharedRef<ISourceControlState>& FileState : ChangelistState->GetFilesStates())
		{
			TSharedPtr<IFileViewTreeItem> FileItem;
			if (TSharedPtr<IChangelistTreeItem>* CachedFileItem = SourceControlItemCache.Find(FileState); CachedFileItem && CachedFileItem->IsValid())
			{
				check((*CachedFileItem)->GetTreeItemType() == IChangelistTreeItem::File);
				FileItem = StaticCastSharedPtr<FFileTreeItem>(*CachedFileItem);
				OnSourceControlCachedItemVisited(FileItem.Get());
			}
			else
			{
				FileItem = MakeShared<FFileTreeItem>(FileState, bBeautifyPaths);
				OnNewSourceControlItemCreated(FileState, FileItem);
			}

			if (bChangelistPassedFilter && FileTextFilter->PassesFilter(*FileItem) && !DuplicateItems.Contains(FileItem.Get()))
			{
				ChangelistTreeItem->AddChild(FileItem.ToSharedRef());
				if (bChangelistSelected)
				{
					AddItemToFileViewWrapper(FileItem);
				}
			}
		}

		// Create or reuse the 'Shelved File' nodes.
		const bool bShelvedChangeslistSelected = SelectedChangelistItem == ShelvedChangelistTreeItem;
		for (const TSharedRef<ISourceControlState>& ShelvedFileState : ChangelistState->GetShelvedFilesStates())
		{
			TSharedPtr<IFileViewTreeItem> ShelvedFileItem;
			if (TSharedPtr<IChangelistTreeItem>* CachedShelvedFileItem = SourceControlItemCache.Find(ShelvedFileState); CachedShelvedFileItem && CachedShelvedFileItem->IsValid())
			{
				check((*CachedShelvedFileItem)->GetTreeItemType() == IChangelistTreeItem::ShelvedFile);
				ShelvedFileItem = StaticCastSharedPtr<FShelvedFileTreeItem>(*CachedShelvedFileItem);
				OnSourceControlCachedItemVisited(ShelvedFileItem.Get());
			}
			else
			{
				ShelvedFileItem = MakeShared<FShelvedFileTreeItem>(ShelvedFileState, bBeautifyPaths);
				OnNewSourceControlItemCreated(ShelvedFileState, ShelvedFileItem);
			}

			if (bChangelistPassedFilter && bShelvedChangelistPassesFilter && FileTextFilter->PassesFilter(*ShelvedFileItem) && !DuplicateItems.Contains(ShelvedFileItem.Get()))
			{
				ShelvedChangelistTreeItem->AddChild(ShelvedFileItem.ToSharedRef());
				if (bShelvedChangeslistSelected)
				{
					AddItemToFileViewWrapper(ShelvedFileItem);
				}
			}
		}
	}

	RemoveDiscardedSourceControlItems();
	ChangelistTreeView->RequestTreeRefresh();

	// If a duplicate item was detected then request a changelists refresh in the next tick 
	bShouldRefresh |= !DuplicateItems.IsEmpty();
}

void SSourceControlChangelistsWidget::OnRefreshUncontrolledChangelistWidgets(int64 CurrUpdateNum, const TFunction<void(TSharedPtr<IFileViewTreeItem>&)>& AddItemToFileView)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SSourceControlChangelistsWidget::OnRefreshUncontrolledChangelistWidgets);

	const bool bBeautifyPaths = true;
	const int64 PrevUpdateNum = CurrUpdateNum - 1;
	int32 DisplayedItemCount = 0;
	int32 VisitedItemCount = 0;
	TArray<IChangelistTreeItem*> DuplicateItems;

	TSharedPtr<IChangelistTreeItem> SelectedChangelistItem = UncontrolledChangelistTreeView->GetNumItemsSelected() > 0 ? UncontrolledChangelistTreeView->GetSelectedItems()[0] : TSharedPtr<IChangelistTreeItem>();

	auto HasUncontrolledChangelistSelected = [&SelectedChangelistItem]()
	{
		return SelectedChangelistItem && SelectedChangelistItem->GetTreeItemType() == IChangelistTreeItem::UncontrolledChangelist;
	};

	auto OnUncontrolledChangelistCachedItemVisited = [CurrUpdateNum, &VisitedItemCount, &DuplicateItems](IChangelistTreeItem* Item)
	{
		if (Item->VisitedUpdateNum < CurrUpdateNum)
		{
			Item->VisitedUpdateNum = CurrUpdateNum;
			++VisitedItemCount;
		}
		else
		{
			// The same uncontrolled item was visited twice. It is likely present in more than one uncontrolled changelist
			// Handle it gracefully by ignoring the duplicate and then request a full changelists refresh for next tick
			DuplicateItems.Emplace(Item);
		}
	};

	auto OnNewUncontrolledChangelistItemCreated = [this, &OnUncontrolledChangelistCachedItemVisited](TSharedPtr<void> Key, TSharedPtr<IChangelistTreeItem> Item)
	{
		OnUncontrolledChangelistCachedItemVisited(Item.Get());
		UncontrolledChangelistItemCache.Emplace(MoveTemp(Key), MoveTemp(Item));
	};

	auto OnNewOfflineFileCreated = [this, &OnUncontrolledChangelistCachedItemVisited](FString Key, TSharedPtr<IFileViewTreeItem> Item)
	{
		OnUncontrolledChangelistCachedItemVisited(Item.Get());
		OfflineFileItemCache.Emplace(MoveTemp(Key), MoveTemp(Item));
	};

	auto AddItemToFileViewWrapper = [&AddItemToFileView, &DisplayedItemCount](TSharedPtr<IFileViewTreeItem>& Item)
	{
		++DisplayedItemCount;
		AddItemToFileView(Item);
	};

	auto RemoveDiscardedUncontrolledChangelistItems = [this, VisitedItemCount, DisplayedItemCount, CurrUpdateNum, HasUncontrolledChangelistSelected]()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SSourceControlChangelistsWidget::RemoveDiscardedUncontrolledChangelistItems);

		check(UncontrolledChangelistItemCache.Num() + OfflineFileItemCache.Num() >= VisitedItemCount); // Cannot visit items that are not cached.

		// Remove the items used to be visited but weren't visited during this iteration.
		int32 DiscardCount = UncontrolledChangelistItemCache.Num() + OfflineFileItemCache.Num() - VisitedItemCount;
		for (auto It = UncontrolledChangelistItemCache.CreateIterator(); It && DiscardCount > 0 ; ++It)
		{
			if (It->Value->VisitedUpdateNum < CurrUpdateNum)
			{
				It.RemoveCurrent();
				--DiscardCount;
			}
		}
		for (auto It = OfflineFileItemCache.CreateIterator(); It && DiscardCount > 0 ; ++It)
		{
			if (It->Value->VisitedUpdateNum < CurrUpdateNum)
			{
				It.RemoveCurrent();
				--DiscardCount;
			}
		}

		// If the uncontrolled changelist feeds the file view.
		if (HasUncontrolledChangelistSelected())
		{
			// Unless the FileTreeNode array was reset because the selected changelist changed, it still contains the union of items to display for this iteration and the item that were displayed last
			// iteration, so we need to remove item that aren't displayed anymore. (The FileTreeNode is not reset between updates to preserve the sort order as much as possible).
			if (FileListNodes.Num() > DisplayedItemCount)
			{
				// Remove items that were not displayed anymore.
				FileListNodes.RemoveAll([CurrUpdateNum](const TSharedPtr<IChangelistTreeItem>& Candidate) { return Candidate->DisplayedUpdateNum < CurrUpdateNum; });
			}
		}
	};

	FUncontrolledChangelistsModule& UncontrolledChangelistModule = FUncontrolledChangelistsModule::Get();
	TArray<TSharedRef<FUncontrolledChangelistState>> UncontrolledChangelistStates = UncontrolledChangelistModule.GetChangelistStates();

	UncontrolledChangelistTreeNodes.Reset(UncontrolledChangelistStates.Num());

	for (const TSharedRef<FUncontrolledChangelistState>& UncontrolledChangelistState : UncontrolledChangelistStates)
	{
		// Create or reuse the 'Uncontrolled Changelist' node.
		TSharedPtr<FUncontrolledChangelistTreeItem> UncontrolledChangelistNode;
		if (TSharedPtr<IChangelistTreeItem>* CachedUncontrolledChangelistItem = UncontrolledChangelistItemCache.Find(UncontrolledChangelistState); CachedUncontrolledChangelistItem && CachedUncontrolledChangelistItem->IsValid())
		{
			check((*CachedUncontrolledChangelistItem)->GetTreeItemType() == IChangelistTreeItem::UncontrolledChangelist);
			UncontrolledChangelistNode = StaticCastSharedPtr<FUncontrolledChangelistTreeItem>(*CachedUncontrolledChangelistItem);
			UncontrolledChangelistNode->RemoveAllChildren(); // Will relink the children against the active filter below.
			OnUncontrolledChangelistCachedItemVisited(UncontrolledChangelistNode.Get());
		}
		else
		{
			UncontrolledChangelistNode = MakeShared<FUncontrolledChangelistTreeItem>(UncontrolledChangelistState);
			OnNewUncontrolledChangelistItemCreated(UncontrolledChangelistState, UncontrolledChangelistNode);
		}

		const bool bChangelistPassedFilter = UncontrolledChangelistTextFilter->PassesFilter(*UncontrolledChangelistNode);
		if (bChangelistPassedFilter)
		{
			UncontrolledChangelistTreeNodes.Add(UncontrolledChangelistNode);
		}

		// Create or reuse 'Uncontrolled' files.
		const bool bUncontrolledChangelistSelected = SelectedChangelistItem == UncontrolledChangelistNode;
		for (const TSharedRef<ISourceControlState>& FileState : UncontrolledChangelistState->GetFilesStates())
		{
			TSharedPtr<IFileViewTreeItem> UncontrolledFileItem;
			if (TSharedPtr<IChangelistTreeItem>* CachedUncontrolledFileItem = UncontrolledChangelistItemCache.Find(FileState); CachedUncontrolledFileItem && CachedUncontrolledFileItem->IsValid())
			{
				check((*CachedUncontrolledFileItem)->GetTreeItemType() == IChangelistTreeItem::File);
				UncontrolledFileItem = StaticCastSharedPtr<FFileTreeItem>(*CachedUncontrolledFileItem);
				OnUncontrolledChangelistCachedItemVisited(UncontrolledFileItem.Get());
			}
			else
			{
				UncontrolledFileItem = MakeShared<FFileTreeItem>(FileState, bBeautifyPaths);
				OnNewUncontrolledChangelistItemCreated(FileState, UncontrolledFileItem);
			}

			if (bChangelistPassedFilter && FileTextFilter->PassesFilter(*UncontrolledFileItem) && !DuplicateItems.Contains(UncontrolledFileItem.Get()))
			{
				UncontrolledChangelistNode->AddChild(UncontrolledFileItem.ToSharedRef());
				if (bUncontrolledChangelistSelected)
				{
					AddItemToFileViewWrapper(UncontrolledFileItem);
				}
			}
		}

		// Create or reuse 'Offline' files.
		for (const FString& Filename : UncontrolledChangelistState->GetOfflineFiles())
		{
			TSharedPtr<IFileViewTreeItem> OfflineFileItem;
			if (TSharedPtr<IFileViewTreeItem>* CachedOfflineFileItem = OfflineFileItemCache.Find(Filename))
			{
				OfflineFileItem = *CachedOfflineFileItem;
				OnUncontrolledChangelistCachedItemVisited(OfflineFileItem.Get());
			}
			else
			{
				OfflineFileItem = MakeShared<FOfflineFileTreeItem>(Filename);
				OnNewOfflineFileCreated(Filename, OfflineFileItem);
			}

			if (bChangelistPassedFilter && FileTextFilter->PassesFilter(*OfflineFileItem) && !DuplicateItems.Contains(OfflineFileItem.Get()))
			{
				UncontrolledChangelistNode->AddChild(OfflineFileItem.ToSharedRef());
				if (bUncontrolledChangelistSelected)
				{
					AddItemToFileViewWrapper(OfflineFileItem);
				}
			}
		}
	}

	RemoveDiscardedUncontrolledChangelistItems();
	UncontrolledChangelistTreeView->RequestTreeRefresh();

	// If a duplicate item was detected then request a changelists refresh in the next tick 
	bShouldRefresh |= !DuplicateItems.IsEmpty();
}

void SSourceControlChangelistsWidget::OnRefreshUI(ERefreshFlags RefreshFlag)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SSourceControlChangelistsWidget::OnRefreshUI);

	// The refresh algorithm tries to only update what changed. This is required to deal smoothly with reasonably large
	// changelists (5000 - 10000 files). The code focus on:
	//   - Detecting added and removed items to do a 'delta' update (important for the file view)
	//   - Preserving the sort order when possible.
	//   - Caching and preserving file timestamps and update them using background task.
	//   - Try performing expensive operations only on visible items.
	// This gives those benefits:
	// - The UI doesn't have to regenerate widget that didn't change.
	// - The selection of items that were displayed before and after an update is naturally preserved.
	// - The sort order is preserved most of the time.
	// - The cache keeps existing item even when filtered out, especially to avoiding to query the file timestamps (very expensive) too often.

	if (!AreChangelistsEnabled())
	{
		ClearChangelistsTree();
		return;
	}

	const int64 PrevUpdateNum = UpdateRequestNum;
	const int64 CurrUpdateNum = ++UpdateRequestNum;
	int32 NewDisplayedItemCount = 0;
	bool bDisplayedIconsPriorityChanged = false;

	auto AddItemToFileView = [this, &NewDisplayedItemCount, &bDisplayedIconsPriorityChanged, PrevUpdateNum, CurrUpdateNum](TSharedPtr<IFileViewTreeItem>& Item)
	{
		// If the items wasn't displayed last update.
		// Or, if FileListNodes was cleared from OnChangelistSelectionChanged
		if (Item->DisplayedUpdateNum != PrevUpdateNum || !FileListNodes.Contains(Item))
		{
			checkfSlow(!FileListNodes.Contains(Item), TEXT("Inserting duplicated items. Something is wrong with the display update number."))

			++NewDisplayedItemCount;
			bUpdateMonitoredFileStatusList = true;

			// Add item at the end, will need to sort the list if the view is sorted.
			FileListNodes.Emplace(Item);

			// That's the first time the item is displayed.
			if (Item->DisplayedUpdateNum == -1)
			{
				Item->SetLastModifiedDateTime(IFileManager::Get().GetTimeStamp(*Item->GetFullPathname()));
			}
		}

		// Check when the file status changes (possibly invalidating sorting)
		int32 IconSortingPriority = Item->GetIconSortingPriority();
		if (Item->DisplayedIconPriority != IconSortingPriority)
		{
			bDisplayedIconsPriorityChanged = true;
			Item->DisplayedIconPriority = IconSortingPriority;
		}

		// Mark this item as 'displayed' at this 'iteration'.
		Item->DisplayedUpdateNum = CurrUpdateNum;
	};

	bool bFileViewUpdated = false;
	if (EnumHasAnyFlags(RefreshFlag, ERefreshFlags::SourceControlChangelists))
	{
		OnRefreshSourceControlWidgets(CurrUpdateNum, AddItemToFileView);
		bFileViewUpdated |= (ChangelistTreeView->GetNumItemsSelected() > 0);
	}
	if (EnumHasAnyFlags(RefreshFlag, ERefreshFlags::UncontrolledChangelists))
	{
		OnRefreshUncontrolledChangelistWidgets(CurrUpdateNum, AddItemToFileView);
		bFileViewUpdated |= (UncontrolledChangelistTreeView->GetNumItemsSelected() > 0);
	}
	if (EnumHasAnyFlags(RefreshFlag, ERefreshFlags::UnsavedAssets))
	{
		OnRefreshUnsavedAssetsWidgets(CurrUpdateNum, AddItemToFileView);
		bFileViewUpdated |= (UnsavedAssetsTreeView->GetNumItemsSelected() > 0);
	}

	if (!bFileViewUpdated)
	{
		// The update didn't touch the file view, but the files (if any) were still displayed during this update interation, bump their displayed update number.
		for (TSharedPtr<IChangelistTreeItem>& Item : FileListNodes)
		{
			Item->DisplayedUpdateNum = CurrUpdateNum;
		}
	}

	if (FilesToSelect.Num() > 0)
	{
		TArray<FString> LocalFilesToSelect(MoveTemp(FilesToSelect));
		SetSelectedFiles(LocalFilesToSelect);
	}

	const bool bNeedSorting = !PrimarySortedColumn.IsNone() && (NewDisplayedItemCount > 0 || (IsFileViewSortedByFileStatusIcon() && bDisplayedIconsPriorityChanged));
	if (bNeedSorting)
	{
		SortFileView();
	}
	
	GetActiveFileListView().RequestListRefresh();
}

void SSourceControlChangelistsWidget::OnSourceControlProviderChanged(ISourceControlProvider& OldProvider, ISourceControlProvider& NewProvider)
{
	OldProvider.UnregisterSourceControlStateChanged_Handle(SourceControlStateChangedDelegateHandle);
	SourceControlStateChangedDelegateHandle = NewProvider.RegisterSourceControlStateChanged_Handle(FSourceControlStateChanged::FDelegate::CreateSP(this, &SSourceControlChangelistsWidget::OnSourceControlStateChanged));

	bSourceControlAvailable = NewProvider.IsAvailable(); // Check if it is connected.
	bShouldRefresh = true;

	if (&NewProvider != &OldProvider)
	{
		if (ChangelistTreeView->GetNumItemsSelected() > 0)
		{
			FileListNodes.Reset();
			GetActiveFileListView().RequestListRefresh();
		}
		ChangelistTreeNodes.Reset();
		SourceControlItemCache.Reset();
		ChangelistTreeView->RequestTreeRefresh();
	}
}

void SSourceControlChangelistsWidget::OnSourceControlStateChanged()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SSourceControlChangelistsWidget::OnSourceControlStateChanged);

	// NOTE: No need to call RequestChangelistsRefresh() to force the SCC to update internal states. We are being invoked because it was update, we just
	//       need to update the UI to reflect those state changes.
	OnRefreshUI(ERefreshFlags::SourceControlChangelists);
}

void SSourceControlChangelistsWidget::OnChangelistsStatusUpdated(const TSharedRef<ISourceControlOperation>& InOperation, ECommandResult::Type InType)
{
	// NOTE: This is invoked when the 'FUpdatePendingChangelistsStatus' completes. No need to refresh the tree views because OnSourceControlStateChanged() is also called.
	OnEndSourceControlOperation(InOperation, InType);
	EndRefreshStatus();
}

void SSourceControlChangelistsWidget::OnUncontrolledChangelistStateChanged()
{
	OnRefreshUI(ERefreshFlags::UncontrolledChangelists);
}

FSourceControlChangelistStatePtr SSourceControlChangelistsWidget::GetCurrentChangelistState() const
{
	if (!ChangelistTreeView)
	{
		return nullptr;
	}

	TArray<TSharedPtr<IChangelistTreeItem>> SelectedItems = ChangelistTreeView->GetSelectedItems();
	if (SelectedItems.Num() != 1 || SelectedItems[0]->GetTreeItemType() != IChangelistTreeItem::Changelist)
	{
		return nullptr;
	}

	return StaticCastSharedPtr<FChangelistTreeItem>(SelectedItems[0])->ChangelistState;
}

FUncontrolledChangelistStatePtr SSourceControlChangelistsWidget::GetCurrentUncontrolledChangelistState() const
{
	if (!UncontrolledChangelistTreeView)
	{
		return nullptr;
	}

	TArray<TSharedPtr<IChangelistTreeItem>> SelectedItems = UncontrolledChangelistTreeView->GetSelectedItems();
	if (SelectedItems.Num() != 1 || SelectedItems[0]->GetTreeItemType() != IChangelistTreeItem::UncontrolledChangelist)
	{
		return nullptr;
	}

	return StaticCastSharedPtr<FUncontrolledChangelistTreeItem>(SelectedItems[0])->UncontrolledChangelistState;
}

FSourceControlChangelistPtr SSourceControlChangelistsWidget::GetCurrentChangelist()
{
	FSourceControlChangelistStatePtr ChangelistState = GetCurrentChangelistState();
	return ChangelistState ? (FSourceControlChangelistPtr)(ChangelistState->GetChangelist()) : nullptr;
}

TOptional<FUncontrolledChangelist> SSourceControlChangelistsWidget::GetCurrentUncontrolledChangelist() const
{
	FUncontrolledChangelistStatePtr UncontrolledChangelistState = GetCurrentUncontrolledChangelistState();
	return UncontrolledChangelistState ? UncontrolledChangelistState->Changelist : TOptional<FUncontrolledChangelist>();
}

FSourceControlChangelistStatePtr SSourceControlChangelistsWidget::GetChangelistStateFromSelection()
{
	TArray<FChangelistTreeItemPtr> SelectedItems = ChangelistTreeView->GetSelectedItems();
	if (SelectedItems.Num() == 0)
	{
		return nullptr;
	}

	FChangelistTreeItemPtr Item = SelectedItems[0];
	while (Item)
	{
		if (Item->GetTreeItemType() == IChangelistTreeItem::Changelist)
		{
			return StaticCastSharedPtr<FChangelistTreeItem>(Item)->ChangelistState;
		}
		Item = Item->GetParent();
	}

	return nullptr;
}

FSourceControlChangelistPtr SSourceControlChangelistsWidget::GetChangelistFromSelection()
{
	FSourceControlChangelistStatePtr ChangelistState = GetChangelistStateFromSelection();
	return ChangelistState ? (FSourceControlChangelistPtr)(ChangelistState->GetChangelist()) : nullptr;
}

void SSourceControlChangelistsWidget::SetSelectedFiles(const TArray<FString>& Filenames)
{
	if (bShouldRefresh || bIsRefreshing)
	{
		FilesToSelect = Filenames;
		return;
	}

	check(Filenames.Num() > 0);

	// Finds the Changelist tree item containing this Filename if it exists.
	auto FindChangelist = [this](const FString& Filename) -> TSharedPtr<IChangelistTreeItem>
	{
		for (const TSharedPtr<IChangelistTreeItem>& Item : ChangelistTreeNodes)
		{
			for (const TSharedPtr<IChangelistTreeItem>& ChildItem : Item->GetChildren())
			{
				if (ChildItem->GetTreeItemType() == IChangelistTreeItem::File)
				{
					const FString& ChildFilename = StaticCastSharedPtr<FFileTreeItem>(ChildItem)->GetFullPathname();
					if (ChildFilename.Compare(Filename, ESearchCase::IgnoreCase) == 0)
					{
						return Item;
					}
				}
			}
		}

		for (const TSharedPtr<IChangelistTreeItem>& Item : UncontrolledChangelistTreeNodes)
		{
			for (const TSharedPtr<IChangelistTreeItem>& ChildItem : Item->GetChildren())
			{
				if (ChildItem->GetTreeItemType() == IChangelistTreeItem::File || ChildItem->GetTreeItemType() == IChangelistTreeItem::OfflineFile)
				{
					const FString& ChildFilename = StaticCastSharedPtr<IFileViewTreeItem>(ChildItem)->GetFullPathname();
					if (ChildFilename.Compare(Filename, ESearchCase::IgnoreCase) == 0)
					{
						return Item;
					}
				}
			}
		}

		return nullptr;
	};

	TSharedPtr<IChangelistTreeItem> FoundChangelistTreeItem = nullptr;
	// Find filename in Changelist, since filenames might not be in same Changelist, start from the last Filename as it might be the last selected one and give it priority
	for (int32 Index = Filenames.Num() - 1; Index >= 0; --Index)
	{
		if (TSharedPtr<IChangelistTreeItem> ChangelistTreeItem = FindChangelist(Filenames[Index]))
		{
			FoundChangelistTreeItem = ChangelistTreeItem;
			break;
		}
	}

	// Clear the current selection.
	GetActiveFileListView().ClearSelection();
	TSharedPtr<IChangelistTreeItem> LastFoundItem;

	// If we found a Changelist, select files.
	if (FoundChangelistTreeItem)
	{
		// To make search faster store all filenames lower case
		TSet<FString> FilenamesLowerCase;
		Algo::Transform(Filenames, FilenamesLowerCase, [](const FString& Filename) { return Filename.ToLower(); });

		for (const TSharedPtr<IChangelistTreeItem>& ChildItem : FoundChangelistTreeItem->GetChildren())
		{
			if (ChildItem->GetTreeItemType() == IChangelistTreeItem::File || ChildItem->GetTreeItemType() == IChangelistTreeItem::OfflineFile)
			{
				const FString ChildFilename = StaticCastSharedPtr<IFileViewTreeItem>(ChildItem)->GetFullPathname().ToLower();
				if (FilenamesLowerCase.Contains(ChildFilename))
				{
					GetActiveFileListView().SetItemSelection(ChildItem, /*bSelected*/true);
					LastFoundItem = ChildItem;
				}
			}
		}

		if (LastFoundItem)
		{
			if (FoundChangelistTreeItem->GetTreeItemType() == IChangelistTreeItem::Changelist)
			{
				// Ensure the area is expanded.
				ChangelistExpandableArea->SetExpanded(true);

				if (ChangelistTreeView->GetNumItemsSelected() == 0 || ChangelistTreeView->GetSelectedItems()[0] != FoundChangelistTreeItem)
				{
					ChangelistTreeView->SetSelection(FoundChangelistTreeItem);
				}
			}
			else if (FoundChangelistTreeItem->GetTreeItemType() == IChangelistTreeItem::UncontrolledChangelist)
			{
				// Ensure the area is expanded.
				UncontrolledChangelistExpandableArea->SetExpanded(true);

				if (UncontrolledChangelistTreeView->GetNumItemsSelected() == 0 || UncontrolledChangelistTreeView->GetSelectedItems()[0] != FoundChangelistTreeItem)
				{
					UncontrolledChangelistTreeView->SetSelection(FoundChangelistTreeItem);
				}
			}
			GetActiveFileListView().RequestScrollIntoView(LastFoundItem);
		}
	}
}

bool SSourceControlChangelistsWidget::HasFilesSelected() const
{
	if (GetActiveFileListView().GetNumItemsSelected() == 0)
	{
		return false;
	}
	if (ChangelistTreeView->GetNumItemsSelected() > 0)
	{
		switch (ChangelistTreeView->GetSelectedItems()[0]->GetTreeItemType())
		{
		case IChangelistTreeItem::Changelist:
			return true; // If this type of changeslist is selected and the file view as item selected, it means files are selected.
			case IChangelistTreeItem::UncontrolledChangelist:
				for (const TSharedPtr<IChangelistTreeItem>& Item : FileListView->GetSelectedItems())
				{
					if (Item->GetTreeItemType() == IChangelistTreeItem::File)
					{
						return true; // Early out.
					}
				}
		default:
			break;
		}
	}
	if (UnsavedAssetsFileListView->GetSelectedItems().Num() > 0)
	{
		return true;
	}

	return false;
}

TArray<FString> SSourceControlChangelistsWidget::GetSelectedFiles()
{
	TArray<FString> Files;
	for (const TSharedPtr<IChangelistTreeItem>& Item : FileListView->GetSelectedItems())
	{
		if (Item->GetTreeItemType() == IChangelistTreeItem::File)
		{
			Files.Add(StaticCastSharedPtr<FFileTreeItem>(Item)->FileState->GetFilename());
		}
	}

	return Files;
}

void SSourceControlChangelistsWidget::GetSelectedFiles(TArray<FString>& OutControlledFiles, TArray<FString>& OutUncontrolledFiles)
{
	for (const TSharedPtr<IChangelistTreeItem>& Item : GetActiveFileListView().GetSelectedItems())
	{
		if (Item->GetTreeItemType() == IChangelistTreeItem::File)
		{
			if (TSharedPtr<IChangelistTreeItem> Parent = Item->GetParent())
			{
				const FString& Filename = StaticCastSharedPtr<FFileTreeItem>(Item)->FileState->GetFilename();

				if (Parent->GetTreeItemType() == IChangelistTreeItem::Changelist)
				{
					OutControlledFiles.Add(Filename);
				}
				else if (Parent->GetTreeItemType() == IChangelistTreeItem::UncontrolledChangelist)
				{
					OutUncontrolledFiles.Add(Filename);
				}
			}
		}
		else if (Item->GetTreeItemType() == IChangelistTreeItem::OfflineFile)
		{
			if (TSharedPtr<IChangelistTreeItem> Parent = Item->GetParent())
			{
				if (Parent->GetTreeItemType() == IChangelistTreeItem::UncontrolledChangelist)
				{
					const FString& Filename = StaticCastSharedPtr<FOfflineFileTreeItem>(Item)->GetFullPathname();
					OutUncontrolledFiles.Add(Filename);
				}
				else if (Parent->GetTreeItemType() == IChangelistTreeItem::UnsavedAssets)
				{
					const FString& Filename = StaticCastSharedPtr<FOfflineFileTreeItem>(Item)->GetFullPathname();

					if (AnyUncontrolledChangelistContainsFilename(Filename))
					{
						OutUncontrolledFiles.Add(Filename);
					}
					else
					{
						OutControlledFiles.Add(Filename);
					}
				}
			}
		}
	}
}

void SSourceControlChangelistsWidget::GetSelectedFileStates(TArray<FSourceControlStateRef>& OutControlledFileStates, TArray<FSourceControlStateRef>& OutUncontrolledFileStates)
{
	TArray<TSharedPtr<IChangelistTreeItem>> SelectedItems = GetActiveFileListView().GetSelectedItems();

	for (const TSharedPtr<IChangelistTreeItem>& Item : SelectedItems)
	{
		if (Item->GetTreeItemType() != IChangelistTreeItem::File)
		{
			continue;
		}

		if (const TSharedPtr<IChangelistTreeItem>& Parent = Item->GetParent())
		{
			if (Parent->GetTreeItemType() == IChangelistTreeItem::Changelist)
			{
				OutControlledFileStates.Add(StaticCastSharedPtr<FFileTreeItem>(Item)->FileState);
			}
			else if (Parent->GetTreeItemType() == IChangelistTreeItem::UncontrolledChangelist)
			{
				OutUncontrolledFileStates.Add(StaticCastSharedPtr<FFileTreeItem>(Item)->FileState);
			}
		}
	}
}

bool SSourceControlChangelistsWidget::HasShelvedFilesSelected() const
{
	return FileListView->GetNumItemsSelected() > 0 &&
	       ChangelistTreeView->GetNumItemsSelected() > 0 &&
	       ChangelistTreeView->GetSelectedItems()[0]->GetTreeItemType() == IChangelistTreeItem::ShelvedChangelist;
}

TArray<FString> SSourceControlChangelistsWidget::GetSelectedShelvedFiles()
{
	TArray<FString> ShelvedFiles;

	for (const TSharedPtr<IChangelistTreeItem>& Item : FileListView->GetSelectedItems())
	{
		if (Item->GetTreeItemType() == IChangelistTreeItem::ShelvedFile)
		{
			ShelvedFiles.Add(StaticCastSharedPtr<FShelvedFileTreeItem>(Item)->FileState->GetFilename());
		}
	}

	// No individual 'shelved file' selected?
	if (ShelvedFiles.IsEmpty())
	{
		// Check if the user selected the 'Shelved Files' changelist.
		for (const TSharedPtr<IChangelistTreeItem>& Item : ChangelistTreeView->GetSelectedItems())
		{
			if (Item->GetTreeItemType() == IChangelistTreeItem::ShelvedChangelist)
			{
				// Add all items of the 'Shelved Files' changelist.
				for (const TSharedPtr<IChangelistTreeItem>& Children : Item->GetChildren())
				{
					if (Children->GetTreeItemType() == IChangelistTreeItem::ShelvedFile)
					{
						ShelvedFiles.Add(StaticCastSharedPtr<FShelvedFileTreeItem>(Children)->FileState->GetFilename());
					}
				}

				break; // UI only allows to select one changelist at the time.
			}
		}
	}

	return ShelvedFiles;
}

void SSourceControlChangelistsWidget::Execute(const FText& Message, const TSharedRef<ISourceControlOperation>& InOperation, const EConcurrency::Type InConcurrency, const FSourceControlOperationComplete& InOperationCompleteDelegate)
{
	return Execute(Message, InOperation, nullptr, TArray<FString>(), InConcurrency, InOperationCompleteDelegate);
}

void SSourceControlChangelistsWidget::Execute(const FText& Message, const TSharedRef<ISourceControlOperation>& InOperation, TSharedPtr<ISourceControlChangelist> InChangelist, const EConcurrency::Type InConcurrency, const FSourceControlOperationComplete& InOperationCompleteDelegate)
{
	return Execute(Message, InOperation, MoveTemp(InChangelist), TArray<FString>(), InConcurrency, InOperationCompleteDelegate);
}

void SSourceControlChangelistsWidget::Execute(const FText& Message, const TSharedRef<ISourceControlOperation>& InOperation, const TArray<FString>& InFiles, EConcurrency::Type InConcurrency, const FSourceControlOperationComplete& InOperationCompleteDelegate)
{
	return Execute(Message, InOperation, nullptr, InFiles, InConcurrency, InOperationCompleteDelegate);
}

void SSourceControlChangelistsWidget::Execute(const FText& Message, const TSharedRef<ISourceControlOperation>& InOperation, TSharedPtr<ISourceControlChangelist> InChangelist, const TArray<FString>& InFiles, EConcurrency::Type InConcurrency, const FSourceControlOperationComplete& InOperationCompleteDelegate)
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	// Start the operation.
	OnStartSourceControlOperation(InOperation, Message);

	if (InConcurrency == EConcurrency::Asynchronous)
	{
		// Pass a weak ptr to the lambda to protect in case the 'this' widget is closed/destroyed before the source control operation completes.
		TWeakPtr<SSourceControlChangelistsWidget> ThisWeak(StaticCastSharedRef<SSourceControlChangelistsWidget>(AsShared()));

		SourceControlProvider.Execute(InOperation, MoveTemp(InChangelist), InFiles, InConcurrency, FSourceControlOperationComplete::CreateLambda(
			[ThisWeak, InOperationCompleteDelegate](const TSharedRef<ISourceControlOperation>& Operation, ECommandResult::Type InResult)
			{
				if (TSharedPtr<SSourceControlChangelistsWidget> ThisPtr = ThisWeak.Pin())
				{
					InOperationCompleteDelegate.ExecuteIfBound(Operation, InResult);
					ThisPtr->OnEndSourceControlOperation(Operation, InResult);
				}
			}));
	}
	else
	{
		SSourceControlCommon::ExecuteChangelistOperationWithSlowTaskWrapper(Message, [&]()
		{
			ECommandResult::Type Result = SourceControlProvider.Execute(InOperation, InChangelist, InFiles, InConcurrency, InOperationCompleteDelegate);
			OnEndSourceControlOperation(InOperation, Result);
		});
	}
}

void SSourceControlChangelistsWidget::ExecuteUncontrolledChangelistOperation(const FText& Message, const TFunction<void()>& UncontrolledOperation)
{
	SSourceControlCommon::ExecuteUncontrolledChangelistOperationWithSlowTaskWrapper(Message, UncontrolledOperation);
}

void SSourceControlChangelistsWidget::OnStartSourceControlOperation(TSharedRef<ISourceControlOperation> Operation, const FText& Message)
{
	RefreshStatus = Message; // TODO: Should have a queue to stack async operations going on to correctly display concurrent async operations.
}

void SSourceControlChangelistsWidget::OnEndSourceControlOperation(const TSharedRef<ISourceControlOperation>& Operation, ECommandResult::Type InType)
{
	RefreshStatus = FText::GetEmpty(); // TODO: Should have a queue to stack async operations going on to correctly display concurrent async operations.
}

FReply SSourceControlChangelistsWidget::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	FText FailureMessage;
	if (InKeyEvent.GetKey() == EKeys::Enter)
	{
		// Submit the currently selected changelist (if any, and if conditions are met)
		if (CanSubmitChangelist(&FailureMessage))
		{
			OnSubmitChangelist();
		}
		else
		{
			FText Title(LOCTEXT("Cannot_Submit_Changelist_From_Key_Title", "Cannot Submit Changelist"));
			FMessageDialog::Open(EAppMsgType::Ok, EAppReturnType::Ok, FailureMessage, Title);
		}

		return FReply::Handled();
	}
	else if (InKeyEvent.GetKey() == EKeys::Delete)
	{
		// Delete the currently selected changelist (if any, and if conditions are met)
		if (CanDeleteChangelist(&FailureMessage))
		{
			OnDeleteChangelist();
		}
		else
		{
			FText Title(LOCTEXT("Cannot_Delete_Changelist_From_Key_Title", "Cannot Delete Changelist"));
			FMessageDialog::Open(EAppMsgType::Ok, EAppReturnType::Ok, FailureMessage, Title);
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SSourceControlChangelistsWidget::OnNewChangelist()
{
	FText ChangelistDescription;
	bool bOk = GetChangelistDescription(
		nullptr,
		LOCTEXT("SourceControl.Changelist.New.Title", "New Changelist..."),
		LOCTEXT("SourceControl.Changelist.New.Label", "Enter a description for the changelist:"),
		ChangelistDescription);

	if (!bOk)
	{
		return;
	}

	TSharedRef<FNewChangelist> NewChangelistOperation = ISourceControlOperation::Create<FNewChangelist>();
	NewChangelistOperation->SetDescription(ChangelistDescription);
	Execute(LOCTEXT("Creating_Changelist", "Creating changelist..."), NewChangelistOperation, EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateLambda(
			[](const TSharedRef<ISourceControlOperation>& Operation, ECommandResult::Type InResult)
			{
				if (InResult == ECommandResult::Succeeded)
				{
					SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Create_Changelist_Succeeded", "Changelist successfully created."), SNotificationItem::CS_Success);
				}
				else if (InResult == ECommandResult::Failed)
				{
					SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Create_Changelist_Failed", "Failed to create the changelist."), SNotificationItem::CS_Fail);
				}
			}));
}

void SSourceControlChangelistsWidget::OnDeleteChangelist()
{
	if (GetCurrentChangelist() == nullptr)
	{
		return;
	}

	TSharedRef<FDeleteChangelist> DeleteChangelistOperation = ISourceControlOperation::Create<FDeleteChangelist>();
	
	Execute(LOCTEXT("Deleting_Changelist", "Deleting changelist..."), DeleteChangelistOperation, GetCurrentChangelist(), EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateLambda(
		[](const TSharedRef<ISourceControlOperation>& Operation, ECommandResult::Type InResult)
		{
			if (InResult == ECommandResult::Succeeded)
			{
				SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Delete_Changelist_Succeeded", "Changelist successfully deleted."), SNotificationItem::CS_Success);
			}
			else if (InResult == ECommandResult::Failed)
			{
				SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Delete_Changelist_Failed", "Failed to delete the selected changelist."), SNotificationItem::CS_Fail);
			}
		}));
}

bool SSourceControlChangelistsWidget::CanDeleteChangelist() const
{
	return CanDeleteChangelist(/*OutFailureMessage*/nullptr);
}

bool SSourceControlChangelistsWidget::CanDeleteChangelist(FText* OutFailureMessage) const
{
	FSourceControlChangelistStatePtr ChangelistState = GetCurrentChangelistState();

	if (ChangelistState == nullptr)
	{
		if (OutFailureMessage)
		{
			*OutFailureMessage = LOCTEXT("Cannot_Delete_No_Changelist", "No changelist selected.");
		}
		return false;
	}
	else if (!ChangelistState->GetChangelist()->CanDelete()) // Check if this changelist is deletable (ex. P4 default one is not deletable).
	{
		if (OutFailureMessage)
		{
			*OutFailureMessage = LOCTEXT("Cannot_Delete_Changelist_Not_Deletable", "The selected changelist cannot be deleted.");
		}
		return false;
	}
	else if (ChangelistState->GetFilesStatesNum() > 0 || ChangelistState->GetShelvedFilesStatesNum() > 0)
	{
		if (OutFailureMessage)
		{
			*OutFailureMessage = LOCTEXT("Cannot_Delete_Changelist_Not_Empty", "The changelist is not empty.");
		}
		return false;
	}
	return true;
}

void SSourceControlChangelistsWidget::OnEditChangelist()
{
	FSourceControlChangelistStatePtr ChangelistState = GetCurrentChangelistState();

	if(ChangelistState == nullptr)
	{
		return;
	}

	FText NewChangelistDescription = ChangelistState->GetDescriptionText();

	bool bOk = GetChangelistDescription(
		nullptr,
		LOCTEXT("SourceControl.Changelist.New.Title2", "Edit Changelist..."),
		LOCTEXT("SourceControl.Changelist.New.Label2", "Enter a new description for the changelist:"),
		NewChangelistDescription);

	if (!bOk)
	{
		return;
	}

	EditChangelistDescription(NewChangelistDescription, ChangelistState);
}

void SSourceControlChangelistsWidget::OnRevertUnchanged()
{
	TSharedRef<FRevertUnchanged> RevertUnchangedOperation = ISourceControlOperation::Create<FRevertUnchanged>();
	Execute(LOCTEXT("Reverting_Unchanged_Files", "Reverting unchanged file(s)..."), RevertUnchangedOperation, GetChangelistFromSelection(), GetSelectedFiles(), EConcurrency::Synchronous, FSourceControlOperationComplete::CreateLambda(
		[](const TSharedRef<ISourceControlOperation>& Operation, ECommandResult::Type InType)
		{
			// NOTE: This operation message should tell how many files were reverted and how many weren't.
			if (Operation->GetResultInfo().ErrorMessages.Num() == 0)
			{
				SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Revert_Unchanged_Files_Succeeded", "Unchanged files were reverted."), SNotificationItem::CS_Success);
			}
			else if (InType == ECommandResult::Failed)
			{
				SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Revert_Unchanged_Files_Failed", "Failed to revert unchanged files."), SNotificationItem::CS_Fail);
			}
		}));
}

bool SSourceControlChangelistsWidget::CanRevertUnchanged()
{
	return HasFilesSelected() || (GetCurrentChangelistState() && GetCurrentChangelistState()->GetFilesStatesNum() > 0);
}

void SSourceControlChangelistsWidget::OnRevert()
{
	FText DialogText;
	FText DialogTitle;

	TArray<FString> SelectedControlledFiles;
	TArray<FString> SelectedUncontrolledFiles;

	GetSelectedFiles(SelectedControlledFiles, SelectedUncontrolledFiles);

	// Apply to the entire changelist only of there are no files selected.
	const bool bApplyOnChangelist = (SelectedControlledFiles.Num() == 0 && SelectedUncontrolledFiles.Num() == 0);

	if (bApplyOnChangelist)
	{
		DialogText = LOCTEXT("SourceControl_ConfirmRevertChangelist", "Are you sure you want to revert this changelist?");
		DialogTitle = LOCTEXT("SourceControl_ConfirmRevertChangelist_Title", "Confirm changelist revert");
	}
	else
	{
		DialogText = LOCTEXT("SourceControl_ConfirmRevertFiles", "Are you sure you want to revert the selected files?");
		DialogTitle = LOCTEXT("SourceControl_ConfirmReverFiles_Title", "Confirm files revert");
	}
	
	EAppReturnType::Type UserConfirmation = FMessageDialog::Open(EAppMsgType::OkCancel, EAppReturnType::Ok, DialogText, DialogTitle);

	if (UserConfirmation != EAppReturnType::Ok)
	{
		return;
	}

	// Can only have one changelist selected at the time in the left split view (either a 'Changelist' or a 'Uncontrolled Changelist')
	if (TSharedPtr<ISourceControlChangelist> SelectedChangelist = GetChangelistFromSelection())
	{
		// No specific files selected, pick all the files in the selected the changelist.
		if (SelectedControlledFiles.IsEmpty())
		{
			// Find all the files in that changelist.
			if (FSourceControlChangelistStatePtr ChangelistState = ISourceControlModule::Get().GetProvider().GetState(SelectedChangelist.ToSharedRef(), EStateCacheUsage::Use))
			{
				Algo::Transform(ChangelistState->GetFilesStates(), SelectedControlledFiles, [](const FSourceControlStateRef& FileState)
				{
					return FileState->GetFilename();
				});
			}
		}

		if (!SelectedControlledFiles.IsEmpty())
		{
			TArray<FString> PackageFiles;
			TArray<FString> NonPackageFiles;
			for (const FString& Filename : SelectedControlledFiles)
			{
				if (FPackageName::IsPackageFilename(Filename))
				{
					PackageFiles.Emplace(Filename);
				}
				else
				{
					NonPackageFiles.Emplace(Filename);
				}
			}

			SSourceControlCommon::ExecuteChangelistOperationWithSlowTaskWrapper(LOCTEXT("Reverting_Files", "Reverting file(s)..."), [&PackageFiles, &NonPackageFiles]()
			{
				const bool bRevertAll = false;
				const bool bReloadWorld = AreWorldReloadsAllowed();
			
				if ((PackageFiles.Num() == 0 || SourceControlHelpers::RevertAndReloadPackages(PackageFiles, bRevertAll, bReloadWorld)) && (NonPackageFiles.Num() == 0 || SourceControlHelpers::RevertFiles(NonPackageFiles)))
				{
					SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Revert_Files_Succeeded", "The selected file(s) were reverted."), SNotificationItem::CS_Success);
				}
				else
				{
					SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Revert_Files_Failed", "Failed to revert the selected file(s)."), SNotificationItem::CS_Fail);
				}
			});
		}
	}
	else if (TSharedPtr<FUncontrolledChangelistState> SelectedUncontrolledChangelist = GetCurrentUncontrolledChangelistState())
	{
		// No individual uncontrolled files were selected, revert all the files from the selected uncontrolled changelist.
		if (SelectedUncontrolledFiles.IsEmpty())
		{
			Algo::Transform(SelectedUncontrolledChangelist->GetFilesStates(), SelectedUncontrolledFiles, [](const FSourceControlStateRef& State) { return State->GetFilename(); });
		}

		// Revert uncontrolled files (if any).
		if (!SelectedUncontrolledFiles.IsEmpty())
		{
			ExecuteUncontrolledChangelistOperation(LOCTEXT("Reverting_Uncontrolled_Files", "Reverting uncontrolled files..."), [&SelectedUncontrolledFiles]()
			{
				if (FUncontrolledChangelistsModule::Get().OnRevert(SelectedUncontrolledFiles))
				{
					SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Revert_Uncontrolled_Files_Succeeded", "The selected uncontrolled file(s) were reverted."), SNotificationItem::CS_Success);
				}
				else
				{
					SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Revert_Uncontrolled_Files_Failed", "Failed to revert the selected uncontrolled file(s)."), SNotificationItem::CS_Fail);
				}
			});
		}
	}
	// No changelist selected (and consequently, no files displayed that could be selected).
}

bool SSourceControlChangelistsWidget::CanRevert()
{
	FSourceControlChangelistStatePtr CurrentChangelistState = GetCurrentChangelistState();
	FUncontrolledChangelistStatePtr CurrentUncontrolledChangelistState = GetCurrentUncontrolledChangelistState();

	return HasFilesSelected()
		|| (CurrentChangelistState.IsValid() && CurrentChangelistState->GetFilesStatesNum() > 0)
		|| (CurrentUncontrolledChangelistState.IsValid() && CurrentUncontrolledChangelistState->GetFilesStates().Num() > 0);
}

void SSourceControlChangelistsWidget::OnShelve()
{
	FSourceControlChangelistStatePtr CurrentChangelist = GetChangelistStateFromSelection();

	if (!CurrentChangelist)
	{
		return;
	}

	FText ChangelistDescription = CurrentChangelist->GetDescriptionText();

	if (ChangelistDescription.IsEmptyOrWhitespace())
	{
		bool bOk = GetChangelistDescription(
			nullptr,
			LOCTEXT("SourceControl.Changelist.NewShelve", "Shelving files..."),
			LOCTEXT("SourceControl.Changelist.NewShelve.Label", "Enter a description for the changelist holding the shelve:"),
			ChangelistDescription);

		if (!bOk)
		{
			// User cancelled entering a changelist description; abort shelve
			return;
		}
	}

	TSharedRef<FShelve> ShelveOperation = ISourceControlOperation::Create<FShelve>();
	ShelveOperation->SetDescription(ChangelistDescription);
	Execute(LOCTEXT("Shelving_Files", "Shelving file(s)..."), ShelveOperation, CurrentChangelist->GetChangelist(), GetSelectedFiles(), EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateLambda(
		[](const TSharedRef<ISourceControlOperation>& Operation, ECommandResult::Type InResult)
		{
			if (InResult == ECommandResult::Succeeded)
			{
				SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Shelve_Files_Succeeded", "The selected file(s) were shelved."), SNotificationItem::CS_Success);
			}
			else if (InResult == ECommandResult::Failed)
			{
				SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Shelve_Files_Failed", "Failed to shelved the selected file(s)."), SNotificationItem::CS_Fail);
			}
		}));
}

void SSourceControlChangelistsWidget::OnUnshelve()
{
	FSourceControlChangelistPtr Changelist = GetChangelistFromSelection();
	TArray<FString> Files = GetSelectedShelvedFiles();

	const FText TextMessage = FText::Format(LOCTEXT("Unshelving_Files", "Unshelving {0}|plural(one=file,other=files)..."), Files.Num());
	const FText TextSucceeded = FText::Format(LOCTEXT("Unshelve_Files_Succeeded", "The selected {0}|plural(one=file,other=files) were unshelved."), Files.Num());
	const FText TextFailed = FText::Format(LOCTEXT("Unshelve_Files_Failed", "Failed to unshelved the selected {0}|plural(one=file,other=files)."), Files.Num());

	int32 NumPackageFiles = Algo::CountIf(Files, [](const FString& File) -> bool { return FPackageName::IsPackageFilename(File); });
	if (NumPackageFiles > 0) // Use ApplyOperationAndReloadPackages to deal with editor file locks.
	{
		SSourceControlCommon::ExecuteChangelistOperationWithSlowTaskWrapper(TextMessage,
			[TextSucceeded, TextFailed, Changelist, Files]()
			{
				auto UnshelveOperation =
					[Changelist, Files](const TArray<FString>& InFilenames) -> bool
					{
						return ISourceControlModule::Get().GetProvider().Execute(ISourceControlOperation::Create<FUnshelve>(), Changelist, Files) == ECommandResult::Succeeded;
					};

				const bool bReloadWorld = AreWorldReloadsAllowed();
				const bool bInteractive = false;

				if (SourceControlHelpers::ApplyOperationAndReloadPackages(Files, UnshelveOperation, bReloadWorld, bInteractive))
				{
					SSourceControlCommon::DisplaySourceControlOperationNotification(TextSucceeded, SNotificationItem::CS_Success);
				}
				else
				{
					SSourceControlCommon::DisplaySourceControlOperationNotification(TextFailed, SNotificationItem::CS_Fail);
				}
			}
		);
	}
	else
	{
		TSharedRef<FUnshelve> UnshelveOperation = ISourceControlOperation::Create<FUnshelve>();
		Execute(TextMessage, UnshelveOperation, Changelist, Files, EConcurrency::Synchronous, FSourceControlOperationComplete::CreateLambda(
			[TextSucceeded, TextFailed](const TSharedRef<ISourceControlOperation>& Operation, ECommandResult::Type InResult)
			{
				if (InResult == ECommandResult::Succeeded)
				{
					SSourceControlCommon::DisplaySourceControlOperationNotification(TextSucceeded, SNotificationItem::CS_Success);
				}
				else if (InResult == ECommandResult::Failed)
				{
					SSourceControlCommon::DisplaySourceControlOperationNotification(TextFailed, SNotificationItem::CS_Fail);
				}
			}));
	}
}

void SSourceControlChangelistsWidget::OnDeleteShelvedFiles()
{
	TSharedRef<FDeleteShelved> DeleteShelvedOperation = ISourceControlOperation::Create<FDeleteShelved>();
	Execute(LOCTEXT("Deleting_Shelved_Files", "Deleting shelved file(s)..."), DeleteShelvedOperation, GetChangelistFromSelection(), GetSelectedShelvedFiles(), EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateLambda(
		[](const TSharedRef<ISourceControlOperation>& Operation, ECommandResult::Type InResult)
		{
			if (InResult == ECommandResult::Succeeded)
			{
				SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Delete_Shelved_Files_Succeeded", "The selected shelved file(s) were deleted."), SNotificationItem::CS_Success);
			}
			else if (InResult == ECommandResult::Failed)
			{
				SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Delete_Shelved_Files_Failed", "Failed to delete the selected shelved file(s)."), SNotificationItem::CS_Fail);
			}
		}));
}

static bool GetChangelistValidationResult(FSourceControlChangelistPtr InChangelist, FString& OutValidationTitleText, FString& OutValidationWarningsText, FString& OutValidationErrorsText)
{
	FSourceControlPreSubmitChangelistValidationExDelegate ValidationDelegate = ISourceControlModule::Get().GetRegisteredPreSubmitChangelistValidationEx();

	FSourceControlValidationDelegateResult ValidationResult;

	bool bValidationResult = true;

	if (ValidationDelegate.ExecuteIfBound(InChangelist, ValidationResult))
	{
		EMessageSeverity::Type MessageSeverity = EMessageSeverity::Info;

		if (ValidationResult.Result == EDataValidationResult::Invalid || ValidationResult.Errors.Num() > 0)
		{
			OutValidationTitleText = LOCTEXT("SourceControl.Submit.ChangelistValidationError", "Changelist validation failed!").ToString();
			bValidationResult = false;
			MessageSeverity = EMessageSeverity::Error;
		}
		else if (ValidationResult.Result == EDataValidationResult::NotValidated || ValidationResult.Warnings.Num() > 0)
		{
			OutValidationTitleText = LOCTEXT("SourceControl.Submit.ChangelistValidationWarning", "Changelist validation has warnings!").ToString();
			MessageSeverity = EMessageSeverity::Warning;
		}
		else
		{
			OutValidationTitleText = LOCTEXT("SourceControl.Submit.ChangelistValidationSuccess", "Changelist validation successful!").ToString();
		}

		auto AppendInfo = [](const TArray<FText>& Info, const FText& HeaderFormat, const FText& OverflowFormat, FString& OutText)
			{
				const int32 MaxNumLinesDisplayed = 5;
				int32 NumLinesDisplayed = 0;

				if (Info.Num() > 0)
				{
					if (OutText.Len() > 0)
					{
						OutText += LINE_TERMINATOR;
					}

					FFormatNamedArguments Args;
					Args.Add(TEXT("Count"), Info.Num());
					OutText += FText::Format(HeaderFormat, Args).ToString();

					for (const FText& Line : Info)
					{
						if (NumLinesDisplayed >= MaxNumLinesDisplayed)
						{
							OutText += LINE_TERMINATOR;
							OutText += OverflowFormat.ToString();
							break;
						}

						OutText += LINE_TERMINATOR;
						OutText += Line.ToString();

						++NumLinesDisplayed;
					}
				}
			};

		AppendInfo(ValidationResult.Errors, LOCTEXT("ValidationErrorsHeader", "Encountered {Count} {Count}|plural(one=error,other=errors):"), LOCTEXT("ValidationErrorsOverflow", "See log for complete list of errors"), OutValidationErrorsText);
		AppendInfo(ValidationResult.Warnings, LOCTEXT("ValidationWarningsHeader", "Encountered {Count} {Count}|plural(one=warning,other=warnings):"), LOCTEXT("ValidationWarningsOverflow", "See log for complete list of warnings"), OutValidationWarningsText);
	}

	return bValidationResult;
}

static bool GetOnPresubmitResult(FSourceControlChangelistStatePtr Changelist, FChangeListDescription& Description)
{
	const TArray<FSourceControlStateRef>& FileStates = Changelist->GetFilesStates();

	FText FailureMsg;
	if (!TryToVirtualizeFilesToSubmit(FileStates, Description.Description, FailureMsg))
	{
		// Setup the notification for operation feedback
		FNotificationInfo Info(FailureMsg);

		Info.Text = LOCTEXT("SCC_Checkin_Failed", "Failed to check in files!");
		Info.ExpireDuration = 8.0f;
		Info.HyperlinkText = LOCTEXT("SCC_Checkin_ShowLog", "Show Message Log");
		Info.Hyperlink = FSimpleDelegate::CreateLambda([]() { FMessageLog("SourceControl").Open(EMessageSeverity::Error, true); });

		TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
		Notification->SetCompletionState(SNotificationItem::CS_Fail);

		return false;
	}

	return true;
}

void SSourceControlChangelistsWidget::OnSubmitChangelist()
{
	FSourceControlChangelistStatePtr ChangelistState = GetCurrentChangelistState();
	
	if (!ChangelistState)
	{
		return;
	}

	// first check if there is a submit override bound
	if (ISourceControlWindowsModule::Get().SubmitOverrideDelegate.IsBound())
	{
		// save the changelist description from the widget to perforce
		const FString Identifier = ChangelistState->GetChangelist()->GetIdentifier();

		SSubmitOverrideParameters SubmitOverrideParameters;
		SubmitOverrideParameters.Description = ChangelistState->GetDescriptionText().ToString();
		SubmitOverrideParameters.ToSubmit.SetSubtype<FString>(Identifier);

		FSubmitOverrideReply SubmitOverrideReply = ISourceControlWindowsModule::Get().SubmitOverrideDelegate.Execute(SubmitOverrideParameters);

		switch (SubmitOverrideReply)
		{
			//////////////////////////////////////////////////////////
			case FSubmitOverrideReply::Handled:
			{
				FNotificationInfo Info(LOCTEXT("SCC_Checkin_SubmitOverride_Succeeded", "Successfully invoked the submit override!"));

				Info.Text = LOCTEXT("SCC_Checkin_SubmitOverride_Succeeded", "Successfully invoked the submit override!");
				Info.ExpireDuration = 8.0f;
				Info.HyperlinkText = LOCTEXT("SCC_Checkin_ShowLog", "Show Message Log");
				Info.Hyperlink = FSimpleDelegate::CreateLambda([]() { FMessageLog("SourceControl").Open(EMessageSeverity::Warning, true); });

				TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
				Notification->SetCompletionState(SNotificationItem::CS_Success);

				this->OnRefreshUI(ERefreshFlags::SourceControlChangelists);
				return;
			}
			
			//////////////////////////////////////////////////////////
			case FSubmitOverrideReply::Error:
			{
				FNotificationInfo Info(LOCTEXT("SCC_Checkin_SubmitOverride_Failed", "Failed to invoke the submit override!"));

				Info.Text = LOCTEXT("SCC_Checkin_SubmitOverride_Failed", "Failed to invoke the submit override!");
				Info.ExpireDuration = 8.0f;
				Info.HyperlinkText = LOCTEXT("SCC_Checkin_ShowLog", "Show Message Log");
				Info.Hyperlink = FSimpleDelegate::CreateLambda([]() { FMessageLog("SourceControl").Open(EMessageSeverity::Error, true); });

				TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
				Notification->SetCompletionState(SNotificationItem::CS_Fail);

				this->OnRefreshUI(ERefreshFlags::SourceControlChangelists);
				return;
			}

			//////////////////////////////////////////////////////////
			case FSubmitOverrideReply::ProviderNotSupported:
			default:
				// continue default flow
				break;
		}
	}

	FString ChangelistValidationTitle;
	FString ChangelistValidationWarningsText;
	FString ChangelistValidationErrorsText;
	bool bValidationResult = GetChangelistValidationResult(ChangelistState->GetChangelist(), ChangelistValidationTitle, ChangelistValidationWarningsText, ChangelistValidationErrorsText);

	// The description from the source control.
	const FText CurrentChangelistDescription = ChangelistState->GetDescriptionText();
	const bool bAskForChangelistDescription = (CurrentChangelistDescription.IsEmptyOrWhitespace());

	// The description possibly updated with the #validated proposed to the user.
	FText ChangelistDescriptionToSubmit = UpdateChangelistDescriptionToSubmitIfNeeded(bValidationResult, CurrentChangelistDescription);

	// The description once edited by the user in the Submit window.
	FText UserEditChangelistDescription = ChangelistDescriptionToSubmit;

	// The title from the window
	FSubmitTitleOverrideParameters SubmitTitleOverrideParameters;
	SubmitTitleOverrideParameters.ToSubmit.SetSubtype<FString>(ChangelistState->GetChangelist()->GetIdentifier());

	const FText Title = ISourceControlWindowsModule::Get().SubmitTitleOverrideDelegate.IsBound()
		? ISourceControlWindowsModule::Get().SubmitTitleOverrideDelegate.Execute(SubmitTitleOverrideParameters)
		: NSLOCTEXT("SourceControl.ConfirmSubmit", "Title", "Confirm Submit");

	TSharedRef<SWindow> NewWindow = SNew(SWindow)
		.Title(Title)
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(800, 700))
		.SupportsMaximize(true)
		.SupportsMinimize(false);

	TSharedRef<SSourceControlSubmitWidget> SourceControlWidget =
		SNew(SSourceControlSubmitWidget)
		.ParentWindow(NewWindow)
		.Items(ChangelistState->GetFilesStates())
		.Description(ChangelistDescriptionToSubmit)
		.ChangeValidationResult(ChangelistValidationTitle)
		.ChangeValidationWarnings(ChangelistValidationWarningsText)
		.ChangeValidationErrors(ChangelistValidationErrorsText)
		.AllowDescriptionChange(true)
		.AllowUncheckFiles(false)
		.AllowKeepCheckedOut(true)
		.AllowSubmit(bValidationResult)
		.AllowSaveAndClose(true);

	NewWindow->SetContent(
		SourceControlWidget
	);

	FSlateApplication::Get().AddModalWindow(NewWindow, NULL);

	bool bSaveDescriptionOnSubmitFailure = false;
	bool bCheckinSuccess = false;

	if (SourceControlWidget->GetResult() == ESubmitResults::SUBMIT_ACCEPTED)
	{
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		FChangeListDescription Description;
		TSharedRef<FCheckIn> SubmitChangelistOperation = ISourceControlOperation::Create<FCheckIn>();
		SubmitChangelistOperation->SetKeepCheckedOut(SourceControlWidget->WantToKeepCheckedOut());

		// Get the changelist description the user had when he hit the 'submit' button.
		SourceControlWidget->FillChangeListDescription(Description);
		UserEditChangelistDescription = Description.Description;

		// Check if any of the presubmit hooks fail. (This might also update the changelist description)
		if (GetOnPresubmitResult(ChangelistState, Description))
		{
			// If the user modified the description, ensure the 'validation tag' wasn't removed if validation is enabled.
			if (!ChangelistDescriptionToSubmit.EqualTo(Description.Description))
			{
				SubmitChangelistOperation->SetDescription(UpdateChangelistDescriptionToSubmitIfNeeded(bValidationResult, Description.Description));
			}
			else
			{
				SubmitChangelistOperation->SetDescription(Description.Description);
			}

			Execute(LOCTEXT("Submitting_Changelist", "Submitting changelist..."), SubmitChangelistOperation, ChangelistState->GetChangelist(), EConcurrency::Synchronous, FSourceControlOperationComplete::CreateLambda(
				[&SubmitChangelistOperation, &bCheckinSuccess](const TSharedRef<ISourceControlOperation>& Operation, ECommandResult::Type InResult)
				{
					if (InResult == ECommandResult::Succeeded)
					{
						TOptional<FNotificationInfo> NotificationInfo = SSourceControlCommon::ConstructSourceControlOperationNotification(SubmitChangelistOperation->GetSuccessMessage());
						if (NotificationInfo)
						{
							NotificationInfo->bUseCopyToClipboard = true;
							SSourceControlCommon::DisplaySourceControlOperationNotification(NotificationInfo.GetValue(), SNotificationItem::CS_Success);
						}

						bCheckinSuccess = true;
					}
					else if (InResult == ECommandResult::Failed)
					{
						SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("SCC_Checkin_Failed", "Failed to check in files!"), SNotificationItem::CS_Fail);
					}
				}));
		}

		// If something went wrong with the submit, try to preserve the changelist edited by the user (if he edited).
		bSaveDescriptionOnSubmitFailure = (!bCheckinSuccess && !UserEditChangelistDescription.EqualTo(ChangelistDescriptionToSubmit));
	}

	if (SourceControlWidget->GetResult() == ESubmitResults::SUBMIT_SAVED || bSaveDescriptionOnSubmitFailure)
	{
		FChangeListDescription Description;
		SourceControlWidget->FillChangeListDescription(Description);
		EditChangelistDescription(Description.Description, ChangelistState);
	}
	
	if (bCheckinSuccess)
	{
		// Clear the description saved by the 'submit window'. Useful when the submit window is opened from the Editor menu rather than the changelist window.
		// Opening the 'submit window' from the Editor menu is intended for source controls that do not support changelists (SVN/Git), but remains available to
		// all source controls at the moment.
		SourceControlWidget->ClearChangeListDescription();
	}
}

FText SSourceControlChangelistsWidget::GetCanSubmitChangelistToolTip() const
{
	FText Error;
	if (!CanSubmitChangelist(&Error))
	{
		return Error;
	}
	return LOCTEXT("SourceControl_SubmitChangeslit_Tooltip", "Submits a changelist");
}

bool SSourceControlChangelistsWidget::CanSubmitChangelist() const
{
	return CanSubmitChangelist(/*OutFailureMessage*/nullptr);
}

bool SSourceControlChangelistsWidget::CanSubmitChangelist(FText* OutFailureMessage) const
{
	FSourceControlChangelistStatePtr Changelist = GetCurrentChangelistState();

	if (Changelist == nullptr)
	{
		if (OutFailureMessage)
		{
			*OutFailureMessage = LOCTEXT("Cannot_Submit_Changelist_No_Selection", "No changelist selected.");
		}
		return false;
	}
	else if (Changelist->GetFilesStatesNum() <= 0)
	{
		if (OutFailureMessage)
		{
			*OutFailureMessage = LOCTEXT("Cannot_Submit_Changelist_No_Files", "The changelist doesn't contain any files to submit.");
		}
		return false;
	}
	else if (ISourceControlWindowsModule::Get().CanSubmitOverrideDelegate.IsBound())
	{
		// save the changelist description from the widget to perforce
		const FString Identifier = Changelist->GetChangelist()->GetIdentifier();

		SSubmitOverrideParameters SubmitOverrideParameters;
		SubmitOverrideParameters.Description = Changelist->GetDescriptionText().ToString();
		SubmitOverrideParameters.ToSubmit.SetSubtype<FString>(Identifier);

		FSubmitOverrideReply SubmitOverrideReply = ISourceControlWindowsModule::Get().CanSubmitOverrideDelegate.Execute(SubmitOverrideParameters, OutFailureMessage);

		switch (SubmitOverrideReply)
		{
		case FSubmitOverrideReply::Handled:
			return true;
		case FSubmitOverrideReply::Error:
			return false;

			//////////////////////////////////////////////////////////
		case FSubmitOverrideReply::ProviderNotSupported:
		default:
			// continue default flow
			break;
		}
	}

	if (Changelist->GetShelvedFilesStatesNum() > 0)
	{
		if (OutFailureMessage)
		{
			*OutFailureMessage = LOCTEXT("Cannot_Submit_Changelist_Has_Shelved_Files", "The changelist contains shelved files.");
		}
		return false;
	}

	return true;
}

void SSourceControlChangelistsWidget::OnValidateChangelist()
{
	FSourceControlChangelistStatePtr ChangelistState = GetCurrentChangelistState();

	if (!ChangelistState)
	{
		return;
	}

	FString ChangelistValidationTitle;
	FString ChangelistValidationWarningsText;
	FString ChangelistValidationErrorsText;
	bool bValidationResult = GetChangelistValidationResult(ChangelistState->GetChangelist(), ChangelistValidationTitle, ChangelistValidationWarningsText, ChangelistValidationErrorsText);

	// Setup the notification for operation feedback
	FNotificationInfo Info(LOCTEXT("SCC_Validation_Success", "Changelist validated"));

	// Override the notification fields for failure ones
	if (!bValidationResult)
	{
		Info.Text = LOCTEXT("SCC_Validation_Failed", "Failed to validate the changelist");
	}

	Info.ExpireDuration = 8.0f;
	Info.HyperlinkText = LOCTEXT("SCC_Validation_ShowLog", "Show Message Log");
	Info.Hyperlink = FSimpleDelegate::CreateLambda([]() { FMessageLog("AssetCheck").Open(EMessageSeverity::Info, true); });

	TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
	Notification->SetCompletionState(bValidationResult ? SNotificationItem::CS_Success : SNotificationItem::CS_Fail);
}

bool SSourceControlChangelistsWidget::CanValidateChangelist() const
{
	FSourceControlChangelistStatePtr Changelist = GetCurrentChangelistState();
	return Changelist != nullptr && Changelist->GetFilesStatesNum() > 0;
}

void SSourceControlChangelistsWidget::OnNewUncontrolledChangelist()
{
	FText UncontrolledChangelistDescription;
	bool bOk = GetChangelistDescription(
		nullptr,
		LOCTEXT("SourceControl.UncontrolledChangelist.New.Title", "New Uncontrolled Changelist..."),
		LOCTEXT("SourceControl.UncontrolledChangelist.New.Label", "Enter a description for the uncontrolled changelist:"),
		UncontrolledChangelistDescription);

	if (!bOk)
	{
		return;
	}

	ExecuteUncontrolledChangelistOperation(LOCTEXT("Creating_Uncontrolled_Changelist", "Creating uncontrolled changelist..."), [&]()
	{
		FUncontrolledChangelistsModule::Get().CreateUncontrolledChangelist(UncontrolledChangelistDescription);
	});
}

void SSourceControlChangelistsWidget::OnEditUncontrolledChangelist()
{
	FUncontrolledChangelistStatePtr UncontrolledChangelistState = GetCurrentUncontrolledChangelistState();

	if (UncontrolledChangelistState == nullptr)
	{
		return;
	}

	FText NewUncontrolledChangelistDescription = UncontrolledChangelistState->GetDisplayText();

	bool bOk = GetChangelistDescription(
		nullptr,
		LOCTEXT("SourceControl.Uncontrolled.Changelist.New.Title2", "Edit Uncontrolled Changelist..."),
		LOCTEXT("SourceControl.Uncontrolled.Changelist.New.Label2", "Enter a new description for the uncontrolled changelist:"),
		NewUncontrolledChangelistDescription);

	if (!bOk)
	{
		return;
	}

	ExecuteUncontrolledChangelistOperation(LOCTEXT("Updating_Uncontrolled_Changelist_Description", "Updating uncontrolled changelist description..."), [&NewUncontrolledChangelistDescription, &UncontrolledChangelistState]()
	{
		FUncontrolledChangelistsModule::Get().EditUncontrolledChangelist(UncontrolledChangelistState->Changelist, NewUncontrolledChangelistDescription);
	});
}

bool SSourceControlChangelistsWidget::CanEditUncontrolledChangelist()
{
	FUncontrolledChangelistStatePtr UncontrolledChangelistState = GetCurrentUncontrolledChangelistState();

	return (UncontrolledChangelistState != nullptr) && !UncontrolledChangelistState->Changelist.IsDefault();
}

void SSourceControlChangelistsWidget::OnDeleteUncontrolledChangelist()
{
	TOptional<FUncontrolledChangelist> UncontrolledChangelist = GetCurrentUncontrolledChangelist();

	if (!UncontrolledChangelist.IsSet())
	{
		return;
	}

	ExecuteUncontrolledChangelistOperation(LOCTEXT("Deleting_Uncontrolled_Changelist", "Deleting uncontrolled changelist..."), [&UncontrolledChangelist]()
	{
		FUncontrolledChangelistsModule::Get().DeleteUncontrolledChangelist(UncontrolledChangelist.GetValue());
	});
}

bool SSourceControlChangelistsWidget::CanDeleteUncontrolledChangelist()
{
	FUncontrolledChangelistStatePtr UncontrolledChangelistState = GetCurrentUncontrolledChangelistState();

	return (UncontrolledChangelistState != nullptr) && !UncontrolledChangelistState->Changelist.IsDefault() && !UncontrolledChangelistState->ContainsFiles();
}

TValueOrError<void, void> SSourceControlChangelistsWidget::TryMoveFiles()
{
	TArray<FString> SelectedControlledFiles;
	TArray<FString> SelectedUncontrolledFiles;
	
	GetSelectedFiles(SelectedControlledFiles, SelectedUncontrolledFiles);

	if (SelectedControlledFiles.IsEmpty() && SelectedUncontrolledFiles.IsEmpty())
	{
		return MakeError();
	}

	const bool bAddNewChangelistEntry = true;
	const bool bAddNewUncontrolledChangelistEntry = AreUncontrolledChangelistsEnabled();

	// Build selection list for changelists
	TArray<SSourceControlDescriptionItem> Items;
	Items.Reset(ChangelistTreeNodes.Num() + UncontrolledChangelistTreeNodes.Num() + (bAddNewChangelistEntry ? 1 : 0));

	if (bAddNewChangelistEntry)
	{
		// First item in the 'Move To' list is always 'new changelist'
		Items.Emplace(
			LOCTEXT("SourceControl_NewChangelistText", "New Changelist"),
			LOCTEXT("SourceControl_NewChangelistDescription", "<enter description here>"),
			/*bCanEditDescription=*/true);
	}

	if (bAddNewUncontrolledChangelistEntry)
	{
		// Second item in the 'Move To' list is 'new uncontrolled changelist' (if enabled)
		Items.Emplace(
			LOCTEXT("SourceControl_NewUncontrolledChangelistText", "New Uncontrolled Changelist"),
			LOCTEXT("SourceControl_NewUncontrolledChangelistDescription", "<enter description here>"),
			/*bCanEditDescription=*/true);
	}

	const bool bCanEditAlreadyExistingChangelistDescription = false;

	for (TSharedPtr<IChangelistTreeItem>& Changelist : ChangelistTreeNodes)
	{
		if (Changelist->GetTreeItemType() == IChangelistTreeItem::Changelist)
		{
			const TSharedPtr<FChangelistTreeItem>& TypedChangelist = StaticCastSharedPtr<FChangelistTreeItem>(Changelist);
			Items.Emplace(TypedChangelist->GetDisplayText(), TypedChangelist->GetDescriptionText(), bCanEditAlreadyExistingChangelistDescription);
		}
	}

	for (TSharedPtr<IChangelistTreeItem>& UncontrolledChangelist : UncontrolledChangelistTreeNodes)
	{
		if (UncontrolledChangelist->GetTreeItemType() == IChangelistTreeItem::UncontrolledChangelist)
		{
			const TSharedPtr<FUncontrolledChangelistTreeItem>& TypedChangelist = StaticCastSharedPtr<FUncontrolledChangelistTreeItem>(UncontrolledChangelist);
			Items.Emplace(TypedChangelist->GetDisplayText(), FText(), bCanEditAlreadyExistingChangelistDescription);
		}
	}

	int32 PickedItem = 0;
	FText ChangelistDescription;
	
	bool bOk = PickChangelistOrNewWithDescription(
		nullptr,
		LOCTEXT("SourceControl.MoveFiles.Title", "Move Files To..."),
		LOCTEXT("SourceControl.MoveFIles.Label", "Target Changelist:"),
		Items,
		PickedItem,
		ChangelistDescription);

	if (!bOk)
	{
		return MakeError();
	}

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	bool bFailed = false;
	
	// Move files to a new changelist
	if (bAddNewChangelistEntry && PickedItem == 0)
	{
		// NOTE: To perform async move, we would need to copy the list of selected uncontrolled files and ensure the list wasn't modified when callback occurs. For now run synchronously.
		TSharedRef<FNewChangelist> NewChangelistOperation = ISourceControlOperation::Create<FNewChangelist>();
		NewChangelistOperation->SetDescription(ChangelistDescription);
		Execute(LOCTEXT("Moving_Files_New_Changelist", "Moving file(s) to a new changelist..."), NewChangelistOperation, SelectedControlledFiles, EConcurrency::Synchronous, FSourceControlOperationComplete::CreateLambda(
			[this, &SelectedUncontrolledFiles, &bFailed](const TSharedRef<ISourceControlOperation>& Operation, ECommandResult::Type InResult)
			{
				if (InResult == ECommandResult::Succeeded)
				{
					// NOTE: Perform uncontrolled move only if the new changelist was created and the controlled file were move.
					if ((!SelectedUncontrolledFiles.IsEmpty()) && static_cast<FNewChangelist&>(Operation.Get()).GetNewChangelist().IsValid())
					{
						FUncontrolledChangelistsModule::Get().MoveFilesToControlledChangelist(SelectedUncontrolledFiles, static_cast<FNewChangelist&>(Operation.Get()).GetNewChangelist(), SSourceControlCommon::OpenConflictDialog);
					}

					SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Move_Files_New_Changelist_Succeeded", "Files were successfully moved to a new changelist."), SNotificationItem::CS_Success);
				}
				if (InResult == ECommandResult::Failed)
				{
					SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Move_Files_New_Changelist_Failed", "Failed to move the file to the new changelist."), SNotificationItem::CS_Fail);
					bFailed = true;
				}
			}));
	}
	else if ((bAddNewUncontrolledChangelistEntry && bAddNewChangelistEntry && PickedItem == 1) || (bAddNewUncontrolledChangelistEntry && !bAddNewChangelistEntry && PickedItem == 0)) // Move files to a new uncontrolled changelist
	{
		ExecuteUncontrolledChangelistOperation(LOCTEXT("Moving_Files_New_Uncontrolled_Changelist", "Moving file(s) to a new uncontrolled changelist..."), [&]()
		{
			TArray<FSourceControlStateRef> SelectedControlledFileStates;
			TArray<FSourceControlStateRef> SelectedUnControlledFileStates;
			FUncontrolledChangelistsModule& UncontrolledChangelistsModule = FUncontrolledChangelistsModule::Get();

			GetSelectedFileStates(SelectedControlledFileStates, SelectedUnControlledFileStates);

			TOptional<FUncontrolledChangelist> NewUncontrolledChangelist = UncontrolledChangelistsModule.CreateUncontrolledChangelist(ChangelistDescription);

			if (!NewUncontrolledChangelist.IsSet())
			{
				SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Move_Files_New_Uncontrolled_Changelist_Failed", "Failed to create a new uncontrolled changelist."), SNotificationItem::CS_Fail);
				bFailed = true;
			}
			else if (!SelectedControlledFileStates.IsEmpty() || !SelectedUnControlledFileStates.IsEmpty())
			{
				UncontrolledChangelistsModule.MoveFilesToUncontrolledChangelist(SelectedControlledFileStates, SelectedUnControlledFileStates, NewUncontrolledChangelist.GetValue());
			}
			// SelectedControlledFiles and/or SelectedUncontrolledFiles can contain UnsavedAssets (IChangelistTreeItem::OfflineFile)
			else
			{
				if (!SelectedControlledFiles.IsEmpty())
				{
					UncontrolledChangelistsModule.MoveFilesToUncontrolledChangelist(SelectedControlledFiles, NewUncontrolledChangelist.GetValue());
				}
				if (!SelectedUncontrolledFiles.IsEmpty())
				{
					UncontrolledChangelistsModule.MoveFilesToUncontrolledChangelist(SelectedUncontrolledFiles, NewUncontrolledChangelist.GetValue());
				}
			}
		});
	}
	else // Move files to an existing changelist or uncontrolled changelist.
	{
		// NOTE: The combo box indices are in this order: New changelist, existing changelist(s), existing uncontrolled changelist(s)
		FChangelistTreeItemPtr MoveDestination;
		int32 ChangelistIndex = (bAddNewChangelistEntry ? PickedItem - 1 : PickedItem);

		if (bAddNewUncontrolledChangelistEntry)
		{
			--ChangelistIndex;
			check(ChangelistIndex >= 0);
		}

		if (ChangelistIndex < ChangelistTreeNodes.Num()) // Move files to a changelist
		{
			MoveDestination = ChangelistTreeNodes[ChangelistIndex];
		}
		else // Move files to an uncontrolled changelist. All uncontrolled CL were listed after the controlled CL in the combo box, compute the offset.
		{
			MoveDestination = UncontrolledChangelistTreeNodes[ChangelistIndex - ChangelistTreeNodes.Num()];
		}

		// Move file to a changelist.
		if (MoveDestination->GetTreeItemType() == IChangelistTreeItem::Changelist)
		{
			FSourceControlChangelistPtr Changelist = StaticCastSharedPtr<FChangelistTreeItem>(MoveDestination)->ChangelistState->GetChangelist();

			if (!SelectedControlledFiles.IsEmpty())
			{
				Execute(LOCTEXT("Moving_File_Between_Changelists", "Moving file(s) to the selected changelist..."), ISourceControlOperation::Create<FMoveToChangelist>(), Changelist, SelectedControlledFiles, EConcurrency::Synchronous, FSourceControlOperationComplete::CreateLambda(
					[&bFailed](const TSharedRef<ISourceControlOperation>& Operation, ECommandResult::Type InResult)
					{
						if (InResult == ECommandResult::Succeeded)
						{
							SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Move_Files_Between_Changelist_Succeeded", "File(s) successfully moved to the selected changelist."), SNotificationItem::CS_Success);
						}
						else if (InResult == ECommandResult::Failed)
						{
							SSourceControlCommon::DisplaySourceControlOperationNotification(LOCTEXT("Move_Files_Between_Changelist_Failed", "Failed to move the file(s) to the selected changelist."), SNotificationItem::CS_Fail);
							bFailed = true;
						}
					}));
			}
			if (!SelectedUncontrolledFiles.IsEmpty())
			{
				ExecuteUncontrolledChangelistOperation(LOCTEXT("Moving_Uncontrolled_Files_To_Changelist", "Moving uncontrolled files..."), [&]()
				{
					FUncontrolledChangelistsModule::Get().MoveFilesToControlledChangelist(SelectedUncontrolledFiles, Changelist, SSourceControlCommon::OpenConflictDialog);
				});
			}
		}
		else if (MoveDestination->GetTreeItemType() == IChangelistTreeItem::UncontrolledChangelist)
		{
			const FUncontrolledChangelist UncontrolledChangelist = StaticCastSharedPtr<FUncontrolledChangelistTreeItem>(MoveDestination)->UncontrolledChangelistState->Changelist;
			
			TArray<FSourceControlStateRef> SelectedControlledFileStates;
			TArray<FSourceControlStateRef> SelectedUnControlledFileStates;

			GetSelectedFileStates(SelectedControlledFileStates, SelectedUnControlledFileStates);

			ExecuteUncontrolledChangelistOperation(LOCTEXT("Moving_Uncontrolled_Changelist_To", "Moving uncontrolled files..."), [&]()
			{
				if ((!SelectedControlledFileStates.IsEmpty()) || (!SelectedUnControlledFileStates.IsEmpty()))
				{
					FUncontrolledChangelistsModule::Get().MoveFilesToUncontrolledChangelist(SelectedControlledFileStates, SelectedUnControlledFileStates, UncontrolledChangelist);
				}
				// SelectedControlledFiles and/or SelectedUncontrolledFiles can contain UnsavedAssets (IChangelistTreeItem::OfflineFile)
				else
				{
					if (!SelectedControlledFiles.IsEmpty())
					{
						FUncontrolledChangelistsModule::Get().MoveFilesToUncontrolledChangelist(SelectedControlledFiles, UncontrolledChangelist);
					}
					if (!SelectedUncontrolledFiles.IsEmpty())
					{
						FUncontrolledChangelistsModule::Get().MoveFilesToUncontrolledChangelist(SelectedUncontrolledFiles, UncontrolledChangelist);
					}
				}
			});
		}
	}
	if (bFailed)
	{
		return MakeError();
	}
	return MakeValue();
}

void SSourceControlChangelistsWidget::OnShowHistory()
{
	TArray<FString> SelectedFiles = GetSelectedFiles();
	if (SelectedFiles.Num() > 0)
	{
		FSourceControlWindows::DisplayRevisionHistory(SelectedFiles);
	}
}

void SSourceControlChangelistsWidget::OnDiffAgainstDepot()
{
	TArray<FString> SelectedFiles = GetSelectedFiles();
	if (SelectedFiles.Num() > 0)
	{
		FSourceControlWindows::DiffAgainstWorkspace(SelectedFiles[0]);
	} 
}

bool SSourceControlChangelistsWidget::CanDiffAgainstDepot()
{
	return FileListView->GetNumItemsSelected() == 1 && HasFilesSelected();
}

void SSourceControlChangelistsWidget::OnDiffAgainstWorkspace()
{
	if (GetSelectedShelvedFiles().Num() > 0)
	{
		FSourceControlStateRef FileState = StaticCastSharedPtr<FShelvedFileTreeItem>(FileListView->GetSelectedItems()[0])->FileState;
		FSourceControlWindows::DiffAgainstShelvedFile(FileState);
	}
}

bool SSourceControlChangelistsWidget::CanDiffAgainstWorkspace()
{
	return FileListView->GetNumItemsSelected() == 1 && HasShelvedFilesSelected();
}

void SSourceControlChangelistsWidget::OnSaveAssetsInChangelist()
{
	TArray<TSharedPtr<IChangelistTreeItem>> Files = FileListNodes;

	TArray<UPackage*> Packages;
	Packages.Reserve(Files.Num()); 
	{
		for (const TSharedPtr<IChangelistTreeItem>& File : Files)
		{
			if (TSharedPtr<IFileViewTreeItem> FileItem = StaticCastSharedPtr<IFileViewTreeItem>(File); FileItem.IsValid())
			{
				FString Filename = FileItem->GetFullPathname();
				FString PackageName = UPackageTools::FilenameToPackageName(Filename);

				if (UPackage* Package = FindPackage(nullptr, *PackageName); Package && Package->IsDirty())
				{
					Packages.Add(Package);
				}
			}
		}
	}
	const bool bOnlyDirty = false;
	const bool bPromptToSave = false;
	FEditorFileUtils::PromptForCheckoutAndSave(Packages, bOnlyDirty, bPromptToSave);

	bShouldRefresh = true;
}

bool SSourceControlChangelistsWidget::CanSaveAssetsInChangelist()
{
	TArray<TSharedPtr<IChangelistTreeItem>> Files = FileListNodes;

	for (const TSharedPtr<IChangelistTreeItem>& File : Files)
	{
		if (TSharedPtr<IFileViewTreeItem> FileItem = StaticCastSharedPtr<IFileViewTreeItem>(File); FileItem.IsValid())
		{
			FString Filename = FileItem->GetFullPathname();
			FString PackageName = UPackageTools::FilenameToPackageName(Filename);

			if (UPackage* Package = FindPackage(nullptr, *PackageName); Package && Package->IsDirty())
			{
				return true;
			}
		}
	}


	return false;
}

TSharedPtr<SWidget> SSourceControlChangelistsWidget::OnOpenContextMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	static const FName MenuName = "SourceControl.ChangelistContextMenu";
	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		UToolMenu* RegisteredMenu = ToolMenus->RegisterMenu(MenuName);
		// Add section so it can be used as insert position for menu extensions
		RegisteredMenu->AddSection("Source Control");
	}

	TArray<TSharedPtr<IChangelistTreeItem>> SelectedChangelistNodes = ChangelistTreeView->GetSelectedItems();
	TArray<TSharedPtr<IChangelistTreeItem>> SelectedUncontrolledChangelistNodes = UncontrolledChangelistTreeView->GetSelectedItems();
	TArray<TSharedPtr<IChangelistTreeItem>> SelectedUnsavedAssetNodes = UnsavedAssetsTreeView->GetSelectedItems();

	bool bHasSelectedChangelist = SelectedChangelistNodes.Num() > 0 &&  SelectedChangelistNodes[0]->GetTreeItemType() == IChangelistTreeItem::Changelist;
	bool bHasSelectedShelvedChangelistNode = SelectedChangelistNodes.Num() > 0 &&  SelectedChangelistNodes[0]->GetTreeItemType() == IChangelistTreeItem::ShelvedChangelist;
	bool bHasSelectedUncontrolledChangelist = SelectedUncontrolledChangelistNodes.Num() > 0 &&  SelectedUncontrolledChangelistNodes[0]->GetTreeItemType() == IChangelistTreeItem::UncontrolledChangelist;
	bool bHasSelectedUnsavedAsset = SelectedUnsavedAssetNodes.Num() > 0;
	bool bHasSelectedFiles = HasFilesSelected();
	bool bHasSelectedShelvedFiles = HasShelvedFilesSelected();
	bool bHasEmptySelection = (!bHasSelectedChangelist && !bHasSelectedFiles && !bHasSelectedShelvedFiles);

	// Build up the menu for a selection
	USourceControlMenuContext* ContextObject = NewObject<USourceControlMenuContext>();
	FToolMenuContext Context(ContextObject);

	// Fill Context Object
	TArray<FString> SelectedControlledFiles;
	TArray<FString> SelectedUncontrolledFiles;
	GetSelectedFiles(SelectedControlledFiles, SelectedUncontrolledFiles);
	ContextObject->SelectedFiles.Append(SelectedControlledFiles);
	ContextObject->SelectedFiles.Append(SelectedUncontrolledFiles);

	ContextObject->SelectedChangelist = GetChangelistFromSelection();

	UToolMenu* Menu = ToolMenus->GenerateMenu(MenuName, Context);

	FToolMenuSection& Section = *Menu->FindSection("Source Control");
	
	if (bHasSelectedUnsavedAsset)
	{
		Section.AddMenuEntry(
			"Save", LOCTEXT("SourceControl_SaveUnsavedAsset", "Save"),
			FText::FormatOrdered(LOCTEXT("SourceControl_SaveUnsavedAsset_Tooltip", "Save unsaved {0}|plural(one=asset,other=assets)"), ContextObject->SelectedFiles.Num()),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Save"),
			FUIAction(FExecuteAction::CreateLambda([this]
			{
				TArray<FString> SelectedFiles;
				GetSelectedFiles(SelectedFiles, SelectedFiles);
				TArray<UPackage*> Packages;
				Packages.Reserve(SelectedFiles.Num());
				{
					for (const FString& Filename : SelectedFiles)
					{
						FString PackageName = UPackageTools::FilenameToPackageName(Filename);
						UPackage* Package = FindPackage(nullptr, *PackageName);
						if (ensure(Package))
						{
							Packages.Add(Package);
						}
						else
						{
							Packages.Add(UPackageTools::LoadPackage(PackageName));
						}
					}
				}
				const bool bOnlyDirty = false;
				const bool bPromptToSave = false;
				FEditorFileUtils::PromptForCheckoutAndSave(Packages, bOnlyDirty, bPromptToSave);

				// Request a changelists refresh in the next tick
				bShouldRefresh = true;

			}), FCanExecuteAction::CreateLambda([this]
			{
				return HasFilesSelected();
			})));

		Section.AddMenuEntry(
			"SaveToChangelist", LOCTEXT("SourceControl_SaveUnsavedAssetToChangelist", "Save To Changelist"),
			FText::FormatOrdered(LOCTEXT("SourceControl_SaveUnsavedAssetToChangelist_Tooltip", "Save {0}|plural(one=this asset,other=these assets) and add {0}|plural(one=it,other=them) to a changelist"), ContextObject->SelectedFiles.Num()),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.ArrowRight"),
			FUIAction(FExecuteAction::CreateLambda([this]
			{
				FPackageSourceControlHelper PackageHelper;
				
				TArray<FString> SelectedFiles;
				GetSelectedFiles(SelectedFiles, SelectedFiles);
				
				TArray<FString> PackageNames;
				TArray<FString> PackageNamesToCheckout;
				for (const FString& Filename : SelectedFiles)
				{
					FString PackageName = UPackageTools::FilenameToPackageName(Filename);
					PackageNames.Add(PackageName);

					// If source control is disabled and the file does not exist then don't try to checkout.
					if (PackageHelper.UseSourceControl() || IPlatformFile::GetPlatformPhysical().FileExists(*Filename))
					{
						PackageNamesToCheckout.Add(PackageName);
					}
				}
				
				if (!PackageHelper.Checkout(PackageNamesToCheckout))
				{
					return;
				}
				
				// Actually save the assets
				TArray<UPackage*> Packages;
				Packages.Reserve(PackageNames.Num());
				{
					for (const FString& PackageName : PackageNames)
					{
						UPackage* Package = FindPackage(nullptr, *PackageName);
						if (ensure(Package))
						{
							Packages.Add(Package);
						}
						else
						{
							Packages.Add(UPackageTools::LoadPackage(PackageName));
						}
					}
				}
				const bool bOnlyDirty = false;
				const bool bPromptToSave = false;
				FEditorFileUtils::PromptForCheckoutAndSave(Packages, bOnlyDirty, bPromptToSave);

				// Move the files only after saving the packages.
				// If we are moving files to an uncontrolled changelist, they will be reverted from source control during the move.
				TryMoveFiles();
				
				// Request a changelists refresh in the next tick
				bShouldRefresh = true;

			}), FCanExecuteAction::CreateLambda([this]
			{
				return HasFilesSelected();
			})));

		Section.AddSeparator("ActionSeparator");
	}
	
	// This should appear only on change lists
	if (bHasSelectedChangelist)
	{
		Section.AddMenuEntry(
			"SubmitChangelist",
			LOCTEXT("SourceControl_SubmitChangelist", "Submit Changelist..."),
			{ this, &SSourceControlChangelistsWidget::GetCanSubmitChangelistToolTip },
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::OnSubmitChangelist),
				FCanExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::CanSubmitChangelist)));

		Section.AddMenuEntry(
			"ValidateChangelist",
			LOCTEXT("SourceControl_ValidateChangelist", "Validate Changelist"), LOCTEXT("SourceControl_ValidateChangeslit_Tooltip", "Validates a changelist"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::OnValidateChangelist), 
				FCanExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::CanValidateChangelist)));

		Section.AddMenuEntry(
			"RevertUnchanged",
			LOCTEXT("SourceControl_RevertUnchanged", "Revert Unchanged"),
			LOCTEXT("SourceControl_Revert_Unchanged_Tooltip", "Reverts unchanged files & changelists"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::OnRevertUnchanged),
				FCanExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::CanRevertUnchanged)));
	}

	if (bHasSelectedChangelist || bHasSelectedUncontrolledChangelist)
	{
		Section.AddMenuEntry(
			"Revert",
			LOCTEXT("SourceControl_Revert", "Revert Files"),
			LOCTEXT("SourceControl_Revert_Tooltip", "Reverts all files in the changelist or from the selection"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::OnRevert),
				FCanExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::CanRevert)));
	}

	if (bHasSelectedChangelist && (bHasSelectedFiles || bHasSelectedShelvedFiles || (bHasSelectedChangelist && (GetCurrentChangelistState()->GetFilesStatesNum() > 0 || GetCurrentChangelistState()->GetShelvedFilesStates().Num() > 0))))
	{
		Section.AddSeparator("ShelveSeparator");
	}

	if (bHasSelectedChangelist && (bHasSelectedFiles || (bHasSelectedChangelist && GetCurrentChangelistState()->GetFilesStatesNum() > 0)))
	{
		Section.AddMenuEntry("Shelve",
			LOCTEXT("SourceControl_Shelve", "Shelve Files"),
			LOCTEXT("SourceControl_Shelve_Tooltip", "Shelves the changelist or the selected files"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::OnShelve)));
	}

	if (bHasSelectedShelvedFiles || bHasSelectedShelvedChangelistNode)
	{
		Section.AddMenuEntry(
			"Unshelve",
			LOCTEXT("SourceControl_Unshelve", "Unshelve Files"),
			LOCTEXT("SourceControl_Unshelve_Tooltip", "Unshelve selected files or changelist"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::OnUnshelve)));

		Section.AddMenuEntry(
			"DeleteShelved",
			LOCTEXT("SourceControl_DeleteShelved", "Delete Shelved Files"),
			LOCTEXT("SourceControl_DeleteShelved_Tooltip", "Delete selected shelved files or all from changelist"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::OnDeleteShelvedFiles)));
	}

	// Shelved files-only operations
	if (bHasSelectedShelvedFiles)
	{
		// Diff against workspace
		Section.AddMenuEntry(
			"DiffAgainstWorkspace",
			LOCTEXT("SourceControl_DiffAgainstWorkspace", "Diff Against Workspace Files..."),
			LOCTEXT("SourceControl_DiffAgainstWorkspace_Tooltip", "Diff shelved file against the (local) workspace file"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::OnDiffAgainstWorkspace),
				FCanExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::CanDiffAgainstWorkspace)));
	}

	if (bHasEmptySelection || bHasSelectedChangelist)
	{
		Section.AddSeparator("ChangelistsSeparator");
	}

	if (bHasSelectedChangelist)
	{
		Section.AddMenuEntry(
			"EditChangelist",
			LOCTEXT("SourceControl_EditChangelist", "Edit Changelist..."),
			LOCTEXT("SourceControl_Edit_Changelist_Tooltip", "Edit a changelist description"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::OnEditChangelist)));

		Section.AddMenuEntry(
			"DeleteChangelist",
			LOCTEXT("SourceControl_DeleteChangelist", "Delete Empty Changelist"),
			LOCTEXT("SourceControl_Delete_Changelist_Tooltip", "Deletes an empty changelist"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::OnDeleteChangelist),
				FCanExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::CanDeleteChangelist)));
	}

	if (bHasSelectedUncontrolledChangelist)
	{
		Section.AddMenuEntry(
			"EditUncontrolledChangelist",
			LOCTEXT("SourceControl_EditUncontrolledChangelist", "Edit Uncontrolled Changelist..."),
			LOCTEXT("SourceControl_Edit_Uncontrolled_Changelist_Tooltip", "Edit an uncontrolled changelist description"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::OnEditUncontrolledChangelist),
				FCanExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::CanEditUncontrolledChangelist)));

		Section.AddMenuEntry(
			"DeleteUncontrolledChangelist",
			LOCTEXT("SourceControl_DeleteUncontrolledChangelist", "Delete Empty Changelist"),
			LOCTEXT("SourceControl_Delete_Uncontrolled_Changelist_Tooltip", "Deletes an empty uncontrolled changelist"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::OnDeleteUncontrolledChangelist),
				FCanExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::CanDeleteUncontrolledChangelist)));
	}

	// Files-only operations
	if(bHasSelectedFiles && !bHasSelectedUnsavedAsset)
	{
		Section.AddSeparator("FilesSeparator");

		Section.AddMenuEntry(
			"MoveFiles", LOCTEXT("SourceControl_MoveFiles", "Move Files To..."),
			LOCTEXT("SourceControl_MoveFiles_Tooltip", "Move Files To A Different Changelist..."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this] { TryMoveFiles(); })));

		Section.AddMenuEntry(
			"ShowHistory",
			LOCTEXT("SourceControl_ShowHistory", "Show History..."),
			LOCTEXT("SourceControl_ShowHistory_ToolTip", "Show File History From Selection..."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::OnShowHistory)));

		Section.AddMenuEntry(
			"DiffAgainstLocalVersion",
			LOCTEXT("SourceControl_DiffAgainstDepot", "Diff Against Depot..."),
			LOCTEXT("SourceControl_DiffAgainstLocal_Tooltip", "Diff local file against depot revision."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::OnDiffAgainstDepot),
				FCanExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::CanDiffAgainstDepot)));
	}

	if (FUncontrolledChangelistsModule::Get().IsEnabled() && !bHasSelectedUnsavedAsset)
	{
		Section.AddSeparator("ReconcileSeparator");

		Section.AddMenuEntry(
			"Reconcile assets",
			LOCTEXT("SourceControl_ReconcileAssets", "Reconcile assets"),
			LOCTEXT("SourceControl_ReconcileAssets_Tooltip", "Look for uncontrolled modification in currently added assets."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([]() { FUncontrolledChangelistsModule::Get().OnReconcileAssets(); })));
	}

	if (bHasSelectedChangelist || bHasSelectedUncontrolledChangelist)
	{
		Section.AddSeparator("SaveAssets");

		Section.AddMenuEntry(
			"Save assets",
			LOCTEXT("SourceControl_SaveAssets", "Save All"),
			LOCTEXT("SourceControl_SaveAssets_Tooltip", "Save all unsaved assets in the selected changelist."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::OnSaveAssetsInChangelist), 
				FCanExecuteAction::CreateSP(this, &SSourceControlChangelistsWidget::CanSaveAssetsInChangelist)));
	}

	return ToolMenus->GenerateWidget(Menu);
}

TSharedRef<STreeView<FChangelistTreeItemPtr>> SSourceControlChangelistsWidget::CreateChangelistTreeView(TArray<TSharedPtr<IChangelistTreeItem>>& ItemSources)
{
	return SNew(STreeView<FChangelistTreeItemPtr>)
		.TreeItemsSource(&ItemSources)
		.OnGenerateRow(this, &SSourceControlChangelistsWidget::OnGenerateRow)
		.OnGetChildren(this, &SSourceControlChangelistsWidget::OnGetChangelistChildren)
		.SelectionMode(ESelectionMode::Single)
		.OnMouseButtonDoubleClick(this, &SSourceControlChangelistsWidget::OnItemDoubleClicked)
		.OnContextMenuOpening(this, &SSourceControlChangelistsWidget::OnOpenContextMenu)
		.OnSelectionChanged(this, &SSourceControlChangelistsWidget::OnChangelistSelectionChanged);
}

TSharedRef<SListView<FChangelistTreeItemPtr>> SSourceControlChangelistsWidget::CreateChangelistFilesView()
{
	USourceControlSettings* Settings = GetMutableDefault<USourceControlSettings>();
	if (!Settings->bShowAssetTypeColumn)
	{
		FileViewHiddenColumnsList.Add(SourceControlFileViewColumn::Type::Id());
	}
	if (!Settings->bShowAssetLastModifiedTimeColumn)
	{
		FileViewHiddenColumnsList.Add(SourceControlFileViewColumn::LastModifiedTimestamp::Id());
	}
	if (!Settings->bShowAssetCheckedOutByColumn)
	{
		FileViewHiddenColumnsList.Add(SourceControlFileViewColumn::CheckedOutByUser::Id());
	}

	TSharedRef<SListView<FChangelistTreeItemPtr>> FileView = SNew(SListView<FChangelistTreeItemPtr>)
		.ListItemsSource(&FileListNodes)
		.OnGenerateRow(this, &SSourceControlChangelistsWidget::OnGenerateRow)
		.SelectionMode(ESelectionMode::Multi)
		.OnContextMenuOpening(this, &SSourceControlChangelistsWidget::OnOpenContextMenu)
		.OnMouseButtonDoubleClick(this, &SSourceControlChangelistsWidget::OnItemDoubleClicked)
		.OnListViewScrolled_Lambda([this](double InScrollOffset) { if (!IsFileViewSortedByFileStatusIcon()) { bUpdateMonitoredFileStatusList = true; } }) // If sorted by status icon, the full list of file is already monitored.
		.OnItemToString_Debug_Lambda([this](FChangelistTreeItemPtr Item) { return static_cast<IFileViewTreeItem*>(Item.Get())->GetName(); })
		.HeaderRow
		(
			SNew(SHeaderRow)
			.CanSelectGeneratedColumn(true)
			.HiddenColumnsList(FileViewHiddenColumnsList)
			.OnHiddenColumnsListChanged(this, &SSourceControlChangelistsWidget::OnFileViewHiddenColumnsListChanged)

			+SHeaderRow::Column(SourceControlFileViewColumn::Icon::Id())
			.DefaultLabel(SourceControlFileViewColumn::Icon::GetDisplayText()) // Displayed in the drop down menu to show/hide columns
			.DefaultTooltip(SourceControlFileViewColumn::Icon::GetToolTipText())
			.ShouldGenerateWidget(true) // Ensure the column cannot be hidden (grayed out in the show/hide drop down menu)
			.FillSized(18)
			.HeaderContentPadding(FMargin(0))
			.SortPriority(this, &SSourceControlChangelistsWidget::GetColumnSortPriority, SourceControlFileViewColumn::Icon::Id())
			.SortMode(this, &SSourceControlChangelistsWidget::GetColumnSortMode, SourceControlFileViewColumn::Icon::Id())
			.OnSort(this, &SSourceControlChangelistsWidget::OnColumnSortModeChanged)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(1, 0)
				[
					SNew(SBox)
					.WidthOverride(16)
					.HeightOverride(16)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Visibility_Lambda([this](){ return GetColumnSortMode(SourceControlFileViewColumn::Icon::Id()) == EColumnSortMode::None ? EVisibility::Visible : EVisibility::Collapsed; })
					[
						SNew(SImage)
						.Image(FRevisionControlStyleManager::Get().GetBrush("RevisionControl.ChangelistsTab"))
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					]
				]
			]

			+SHeaderRow::Column(SourceControlFileViewColumn::Shelve::Id())
			.DefaultLabel(SourceControlFileViewColumn::Shelve::GetDisplayText()) // Displayed in the drop down menu to show/hide columns
			.DefaultTooltip(SourceControlFileViewColumn::Shelve::GetToolTipText())
			.ShouldGenerateWidget(true) // Ensure the column cannot be hidden (grayed out in the show/hide drop down menu)
			.FillSized(18)
			.HeaderContentPadding(FMargin(0))
			.SortPriority(this, &SSourceControlChangelistsWidget::GetColumnSortPriority, SourceControlFileViewColumn::Shelve::Id())
			.SortMode(this, &SSourceControlChangelistsWidget::GetColumnSortMode, SourceControlFileViewColumn::Shelve::Id())
			.OnSort(this, &SSourceControlChangelistsWidget::OnColumnSortModeChanged)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(1, 0)
				[
					SNew(SBox)
					.WidthOverride(16)
					.HeightOverride(16)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Visibility_Lambda([this](){ return GetColumnSortMode(SourceControlFileViewColumn::Shelve::Id()) == EColumnSortMode::None ? EVisibility::Visible : EVisibility::Collapsed; })
					[
						SNew(SImage)
						.Image(FRevisionControlStyleManager::Get().GetBrush("RevisionControl.Shelved"))
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					]
				]
			]

			+SHeaderRow::Column(SourceControlFileViewColumn::Name::Id())
			.DefaultLabel(SourceControlFileViewColumn::Name::GetDisplayText())
			.DefaultTooltip(SourceControlFileViewColumn::Name::GetToolTipText())
			.ShouldGenerateWidget(true) // Ensure the column cannot be hidden (grayed out in the show/hide drop down menu)
			.FillWidth(1.5f)
			.SortPriority(this, &SSourceControlChangelistsWidget::GetColumnSortPriority, SourceControlFileViewColumn::Name::Id())
			.SortMode(this, &SSourceControlChangelistsWidget::GetColumnSortMode, SourceControlFileViewColumn::Name::Id())
			.OnSort(this, &SSourceControlChangelistsWidget::OnColumnSortModeChanged)

			+SHeaderRow::Column(SourceControlFileViewColumn::Path::Id())
			.DefaultLabel(SourceControlFileViewColumn::Path::GetDisplayText())
			.DefaultTooltip(SourceControlFileViewColumn::Path::GetToolTipText())
			.ShouldGenerateWidget(true) // Ensure the column cannot be hidden (grayed out in the show/hide drop down menu)
			.FillWidth(3.5f)
			.SortPriority(this, &SSourceControlChangelistsWidget::GetColumnSortPriority, SourceControlFileViewColumn::Path::Id())
			.SortMode(this, &SSourceControlChangelistsWidget::GetColumnSortMode, SourceControlFileViewColumn::Path::Id())
			.OnSort(this, &SSourceControlChangelistsWidget::OnColumnSortModeChanged)

			+SHeaderRow::Column(SourceControlFileViewColumn::Type::Id())
			.DefaultLabel(SourceControlFileViewColumn::Type::GetDisplayText())
			.DefaultTooltip(SourceControlFileViewColumn::Type::GetToolTipText())
			.FillWidth(2.0f)
			.SortPriority(this, &SSourceControlChangelistsWidget::GetColumnSortPriority, SourceControlFileViewColumn::Type::Id())
			.SortMode(this, &SSourceControlChangelistsWidget::GetColumnSortMode, SourceControlFileViewColumn::Type::Id())
			.OnSort(this, &SSourceControlChangelistsWidget::OnColumnSortModeChanged)

			+SHeaderRow::Column(SourceControlFileViewColumn::LastModifiedTimestamp::Id())
			.DefaultLabel(SourceControlFileViewColumn::LastModifiedTimestamp::GetDisplayText())
			.DefaultTooltip(SourceControlFileViewColumn::LastModifiedTimestamp::GetToolTipText())
			.FillWidth(1.5f)
			.SortPriority(this, &SSourceControlChangelistsWidget::GetColumnSortPriority, SourceControlFileViewColumn::LastModifiedTimestamp::Id())
			.SortMode(this, &SSourceControlChangelistsWidget::GetColumnSortMode, SourceControlFileViewColumn::LastModifiedTimestamp::Id())
			.OnSort(this, &SSourceControlChangelistsWidget::OnColumnSortModeChanged)

			+SHeaderRow::Column(SourceControlFileViewColumn::CheckedOutByUser::Id())
			.DefaultLabel(SourceControlFileViewColumn::CheckedOutByUser::GetDisplayText())
			.DefaultTooltip(SourceControlFileViewColumn::CheckedOutByUser::GetToolTipText())
			.FillWidth(1.0f)
			.SortPriority(this, &SSourceControlChangelistsWidget::GetColumnSortPriority, SourceControlFileViewColumn::CheckedOutByUser::Id())
			.SortMode(this, &SSourceControlChangelistsWidget::GetColumnSortMode, SourceControlFileViewColumn::CheckedOutByUser::Id())
			.OnSort(this, &SSourceControlChangelistsWidget::OnColumnSortModeChanged));
	
	return FileView;
}

TSharedRef<SListView<FChangelistTreeItemPtr>> SSourceControlChangelistsWidget::CreateUnsavedAssetsFilesView()
{
	TSharedRef<SListView<FChangelistTreeItemPtr>> FileView = SNew(SListView<FChangelistTreeItemPtr>)
		.ListItemsSource(&FileListNodes)
		.OnGenerateRow(this, &SSourceControlChangelistsWidget::OnGenerateRow)
		.SelectionMode(ESelectionMode::Multi)
		.OnContextMenuOpening(this, &SSourceControlChangelistsWidget::OnOpenContextMenu)
		.HeaderRow
		(
			SNew(SHeaderRow)
			.CanSelectGeneratedColumn(true)
			.HiddenColumnsList(FileViewHiddenColumnsList)
			.OnHiddenColumnsListChanged(this, &SSourceControlChangelistsWidget::OnFileViewHiddenColumnsListChanged)

			+SHeaderRow::Column(SourceControlFileViewColumn::Name::Id())
			.DefaultLabel(SourceControlFileViewColumn::Name::GetDisplayText())
			.DefaultTooltip(SourceControlFileViewColumn::Name::GetToolTipText())
			.ShouldGenerateWidget(true) // Ensure the column cannot be hidden (grayed out in the show/hide drop down menu)
			.FillWidth(1.5f)
			.SortPriority(this, &SSourceControlChangelistsWidget::GetColumnSortPriority, SourceControlFileViewColumn::Name::Id())
			.SortMode(this, &SSourceControlChangelistsWidget::GetColumnSortMode, SourceControlFileViewColumn::Name::Id())
			.OnSort(this, &SSourceControlChangelistsWidget::OnColumnSortModeChanged)

			+SHeaderRow::Column(SourceControlFileViewColumn::Path::Id())
			.DefaultLabel(SourceControlFileViewColumn::Path::GetDisplayText())
			.DefaultTooltip(SourceControlFileViewColumn::Path::GetToolTipText())
			.ShouldGenerateWidget(true) // Ensure the column cannot be hidden (grayed out in the show/hide drop down menu)
			.FillWidth(3.5f)
			.SortPriority(this, &SSourceControlChangelistsWidget::GetColumnSortPriority, SourceControlFileViewColumn::Path::Id())
			.SortMode(this, &SSourceControlChangelistsWidget::GetColumnSortMode, SourceControlFileViewColumn::Path::Id())
			.OnSort(this, &SSourceControlChangelistsWidget::OnColumnSortModeChanged)

			+SHeaderRow::Column(SourceControlFileViewColumn::Type::Id())
			.DefaultLabel(SourceControlFileViewColumn::Type::GetDisplayText())
			.DefaultTooltip(SourceControlFileViewColumn::Type::GetToolTipText())
			.ShouldGenerateWidget(true) // Ensure the column cannot be hidden (grayed out in the show/hide drop down menu)
			.FillWidth(3.5f)
			.SortPriority(this, &SSourceControlChangelistsWidget::GetColumnSortPriority, SourceControlFileViewColumn::Type::Id())
			.SortMode(this, &SSourceControlChangelistsWidget::GetColumnSortMode, SourceControlFileViewColumn::Type::Id())
			.OnSort(this, &SSourceControlChangelistsWidget::OnColumnSortModeChanged)

			+SHeaderRow::Column(SourceControlFileViewColumn::Discard::Id())
			.DefaultLabel(SourceControlFileViewColumn::Discard::GetDisplayText())
			.DefaultTooltip(SourceControlFileViewColumn::Discard::GetToolTipText())
			.ShouldGenerateWidget(true) // Ensure the column cannot be hidden (grayed out in the show/hide drop down menu)
			.FixedWidth(18)
			.HeaderContentPadding(FMargin(0))
			.HAlignHeader(HAlign_Right)
			[
				SNew(SBox)
					.WidthOverride(18)
					.HeightOverride(16)
			]);
	return FileView;
}

void SSourceControlChangelistsWidget::OnFileViewHiddenColumnsListChanged()
{
	if (FileListView && FileListView->GetHeaderRow())
	{
		USourceControlSettings* Settings = GetMutableDefault<USourceControlSettings>();
		Settings->bShowAssetTypeColumn = true;
		Settings->bShowAssetCheckedOutByColumn = true;
		Settings->bShowAssetLastModifiedTimeColumn = true;

		for (const FName& ColumnId : FileListView->GetHeaderRow()->GetHiddenColumnIds())
		{
			if (ColumnId == SourceControlFileViewColumn::Type::Id())
			{
				Settings->bShowAssetTypeColumn = false;
			}
			else if (ColumnId == SourceControlFileViewColumn::LastModifiedTimestamp::Id())
			{
				Settings->bShowAssetLastModifiedTimeColumn = false;
			}
			else if (ColumnId == SourceControlFileViewColumn::CheckedOutByUser::Id())
			{
				Settings->bShowAssetCheckedOutByColumn = false;
			}
		}
		Settings->SaveConfig();
	}
}

TSharedRef<ITableRow> SSourceControlChangelistsWidget::OnGenerateRow(TSharedPtr<IChangelistTreeItem> InTreeItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	switch (InTreeItem->GetTreeItemType())
	{
	case IChangelistTreeItem::Changelist:
		return SNew(SChangelistTableRow, OwnerTable)
			.TreeItemToVisualize(InTreeItem)
			.HighlightText_Lambda([this]() { return ChangelistExpandableArea->GetSearchedText(); })
			.OnPostDrop_Lambda([this] { OnRefreshUI(ERefreshFlags::All); });

	case IChangelistTreeItem::UncontrolledChangelist:
		return SNew(SUncontrolledChangelistTableRow, OwnerTable)
			.TreeItemToVisualize(InTreeItem)
			.HighlightText_Lambda([this]() { return UncontrolledChangelistExpandableArea->GetSearchedText(); })
			.OnPostDrop_Lambda([this] { OnRefreshUI(ERefreshFlags::All); });

	case IChangelistTreeItem::File:
		bUpdateMonitoredFileStatusList = true;
		return SNew(SFileTableRow, OwnerTable)
			.TreeItemToVisualize(InTreeItem)
			.HighlightText_Lambda([this]() { return FileSearchBox->GetText(); })
			.PathFlags_Lambda([this]() { return bShowingContentVersePath ? SourceControlFileViewColumn::EPathFlags::ShowingVersePath : SourceControlFileViewColumn::EPathFlags::Default; })
			.OnDragDetected(this, &SSourceControlChangelistsWidget::OnFilesDragged);

	case IChangelistTreeItem::OfflineFile:
		bUpdateMonitoredFileStatusList = true;
		return SNew(SOfflineFileTableRow, OwnerTable)
			.TreeItemToVisualize(InTreeItem)
			.HighlightText_Lambda([this]() { return FileSearchBox->GetText(); })
			.PathFlags_Lambda([this]() { return bShowingContentVersePath ? SourceControlFileViewColumn::EPathFlags::ShowingVersePath : SourceControlFileViewColumn::EPathFlags::Default; })
			.OnDragDetected(this, &SSourceControlChangelistsWidget::OnUnsavedAssetsDragged);

	case IChangelistTreeItem::ShelvedChangelist:
		return SNew(SShelvedFilesTableRow, OwnerTable)
			.TreeItemToVisualize(InTreeItem)
			.HighlightText_Lambda([this]() { return ChangelistExpandableArea->GetSearchedText(); });

	case IChangelistTreeItem::ShelvedFile:
		bUpdateMonitoredFileStatusList = true;
		return SNew(SFileTableRow, OwnerTable)
			.TreeItemToVisualize(InTreeItem)
			.HighlightText_Lambda([this]() { return FileSearchBox->GetText(); })
			.PathFlags_Lambda([this]() { return bShowingContentVersePath ? SourceControlFileViewColumn::EPathFlags::ShowingVersePath : SourceControlFileViewColumn::EPathFlags::Default; });

	case IChangelistTreeItem::UnsavedAssets:
		return SNew(SUnsavedAssetsTableRow, OwnerTable);
		
	default:
		checkNoEntry();
	};

	return SNew(STableRow<TSharedPtr<FString>>, OwnerTable);
}

FReply SSourceControlChangelistsWidget::OnFilesDragged(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && !FileListView->GetSelectedItems().IsEmpty())
	{
		TSharedRef<FSCCFileDragDropOp> Operation = MakeShared<FSCCFileDragDropOp>();

		for (const FChangelistTreeItemPtr& InTreeItem : FileListView->GetSelectedItems())
		{
			if (InTreeItem->GetTreeItemType() == IChangelistTreeItem::File)
			{
				TSharedRef<FFileTreeItem> FileTreeItem = StaticCastSharedRef<FFileTreeItem>(InTreeItem.ToSharedRef());
				FSourceControlStateRef FileState = FileTreeItem->FileState;

				if (FileTreeItem->GetParent()->GetTreeItemType() == IChangelistTreeItem::UncontrolledChangelist)
				{
					Operation->UncontrolledFiles.Add(MoveTemp(FileState));
				}
				else
				{
					Operation->Files.Add(MoveTemp(FileState));
				}
			}
		}
		
		Operation->Construct();

 		return FReply::Handled().BeginDragDrop(Operation);
	}

	return FReply::Unhandled();
}

FReply SSourceControlChangelistsWidget::OnUnsavedAssetsDragged(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && !UnsavedAssetsFileListView->GetSelectedItems().IsEmpty())
	{
		TSharedRef<FSCCFileDragDropOp> Operation = MakeShared<FSCCFileDragDropOp>();

		for (const FChangelistTreeItemPtr& InTreeItem : UnsavedAssetsFileListView->GetSelectedItems())
		{
			if (InTreeItem->GetTreeItemType() == IChangelistTreeItem::OfflineFile)
			{
				TSharedRef<FOfflineFileTreeItem> FileTreeItem = StaticCastSharedRef<FOfflineFileTreeItem>(InTreeItem.ToSharedRef());
				Operation->OfflineFiles.Emplace(FileTreeItem->GetFullPathname());
			}
		}

		Operation->Construct();

		// Initiates drag-drop operation.
		return FReply::Handled().BeginDragDrop(Operation);
	}

	return FReply::Unhandled();
}

void SSourceControlChangelistsWidget::OnGetChangelistChildren(FChangelistTreeItemPtr InParent, TArray<FChangelistTreeItemPtr>& OutChildren)
{
	if (InParent->GetTreeItemType() == IChangelistTreeItem::Changelist)
	{
		const FChangelistTreeItem* ChangelistItem = static_cast<const FChangelistTreeItem*>(InParent.Get());
		if (ChangelistItem->GetShelvedFileCount() > 0 && ChangelistItem->ShelvedChangelistItem->GetParent() != nullptr)
		{
			OutChildren.Add(ChangelistItem->ShelvedChangelistItem); // Add the 'Shelved Files' only if is is linked to its parent (has items and is not filtered out).
		}
	}
	else if (InParent->GetTreeItemType() == IChangelistTreeItem::UncontrolledChangelist)
	{
		// Uncontrolled changelist nodes do not have children at the moment.
	}
	else if (InParent->GetTreeItemType() == IChangelistTreeItem::UnsavedAssets)
	{
		// The unsaved assets node does not have children at the moment.
	}
}

void SSourceControlChangelistsWidget::OnItemDoubleClicked(TSharedPtr<IChangelistTreeItem> Item)
{
	if (Item->GetTreeItemType() == IChangelistTreeItem::File || Item->GetTreeItemType() == IChangelistTreeItem::OfflineFile)
	{
		const FString& Filename = StaticCastSharedPtr<IFileViewTreeItem>(Item)->GetFullPathname();
		ISourceControlWindowsModule::Get().OnChangelistFileDoubleClicked().Broadcast(Filename);
	}
	else if (Item->GetTreeItemType() == IChangelistTreeItem::Changelist)
	{
		// Submit the currently selected changelists if conditions are met.
		FText FailureMessage;
		if (CanSubmitChangelist(&FailureMessage))
		{
			OnSubmitChangelist();
		}
		else
		{
			FMessageDialog::Open(EAppMsgType::Ok, EAppReturnType::Ok, FailureMessage, LOCTEXT("Cannot_Submit_Changelist_Title", "Cannot Submit Changelist"));
		}
	}
}

void SSourceControlChangelistsWidget::OnChangelistSelectionChanged(TSharedPtr<IChangelistTreeItem> SelectedChangelist, ESelectInfo::Type SelectionType)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SSourceControlChangelistsWidget::OnChangelistSelectionChanged);

	FileListNodes.Reset();
	GetActiveFileListView().RequestListRefresh();

	bUpdateMonitoredFileStatusList = true;

	if (SelectedChangelist) // Can be a Changelist, Uncontrolled Changelist or Shelved Changelist
	{
		IChangelistTreeItem::TreeItemType ChangelistType = SelectedChangelist->GetTreeItemType();
		switch (ChangelistType)
		{
			case IChangelistTreeItem::Changelist:
			case IChangelistTreeItem::ShelvedChangelist:
				UncontrolledChangelistTreeView->ClearSelection(); // Don't have a changelists selected at the same time than an uncontrolled one, they share the same file view.
				UnsavedAssetsTreeView->ClearSelection();
				FileListViewSwitcher->SetActiveWidget(FileListView.ToSharedRef());
				OnRefreshUI(ERefreshFlags::SourceControlChangelists);
				break;

			case IChangelistTreeItem::UncontrolledChangelist:
				ChangelistTreeView->ClearSelection();
				UnsavedAssetsTreeView->ClearSelection();
				FileListViewSwitcher->SetActiveWidget(FileListView.ToSharedRef());
				OnRefreshUI(ERefreshFlags::UncontrolledChangelists);
				break;

			case IChangelistTreeItem::UnsavedAssets:
				ChangelistTreeView->ClearSelection();
				UncontrolledChangelistTreeView->ClearSelection();
				FileListViewSwitcher->SetActiveWidget(UnsavedAssetsFileListView.ToSharedRef());
				OnRefreshUI(ERefreshFlags::UnsavedAssets);
				break;
			
			default:
				break;
		}
	}
}

EColumnSortPriority::Type SSourceControlChangelistsWidget::GetColumnSortPriority(const FName ColumnId) const
{
	if (ColumnId == PrimarySortedColumn)
	{
		return EColumnSortPriority::Primary;
	}
	else if (ColumnId == SecondarySortedColumn)
	{
		return EColumnSortPriority::Secondary;
	}

	return EColumnSortPriority::Max; // No specific priority.
}

EColumnSortMode::Type SSourceControlChangelistsWidget::GetColumnSortMode(const FName ColumnId) const
{
	if (ColumnId == PrimarySortedColumn)
	{
		return PrimarySortMode;
	}
	else if (ColumnId == SecondarySortedColumn)
	{
		return SecondarySortMode;
	}

	return EColumnSortMode::None;
}

void SSourceControlChangelistsWidget::OnColumnSortModeChanged(const EColumnSortPriority::Type InSortPriority, const FName& InColumnId, const EColumnSortMode::Type InSortMode)
{
	if (InSortPriority == EColumnSortPriority::Primary)
	{
		PrimarySortedColumn = InColumnId;
		PrimarySortMode = InSortMode;

		if (InColumnId == SecondarySortedColumn) // Cannot be primary and secondary at the same time.
		{
			SecondarySortedColumn = FName();
			SecondarySortMode = EColumnSortMode::None;
		}
	}
	else if (InSortPriority == EColumnSortPriority::Secondary)
	{
		SecondarySortedColumn = InColumnId;
		SecondarySortMode = InSortMode;
	}

	if (IsFileViewSortedByFileStatusIcon() || IsFileViewSortedByFileShelveIcon())
	{
		bUpdateMonitoredFileStatusList = true;
	}

	SortFileView();
	GetActiveFileListView().RequestListRefresh();
}

bool SSourceControlChangelistsWidget::IsFileViewSortedByFileStatusIcon() const
{
	return PrimarySortedColumn == SourceControlFileViewColumn::Icon::Id() || SecondarySortedColumn == SourceControlFileViewColumn::Icon::Id();
};

bool SSourceControlChangelistsWidget::IsFileViewSortedByFileShelveIcon() const
{
	return PrimarySortedColumn == SourceControlFileViewColumn::Shelve::Id() || SecondarySortedColumn == SourceControlFileViewColumn::Shelve::Id();
};

bool SSourceControlChangelistsWidget::IsFileViewSortedByLastModifiedTimestamp() const
{
	return PrimarySortedColumn == SourceControlFileViewColumn::LastModifiedTimestamp::Id() || SecondarySortedColumn == SourceControlFileViewColumn::LastModifiedTimestamp::Id();
}

void SSourceControlChangelistsWidget::SortFileView()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SSourceControlChangelistsWidget::SortFileView);

	if (PrimarySortedColumn.IsNone() || FileListNodes.IsEmpty())
	{
		return; // No column selected for sorting or nothing to sort.
	}

	const SourceControlFileViewColumn::EPathFlags PathFlags =
		bShowingContentVersePath ? SourceControlFileViewColumn::EPathFlags::ShowingVersePath : SourceControlFileViewColumn::EPathFlags::Default;

	TFunction<int32(const IFileViewTreeItem&, const IFileViewTreeItem&)> PrimaryCompare = SourceControlFileViewColumn::GetColumnComparer(PrimarySortedColumn, PathFlags);
	TFunction<int32(const IFileViewTreeItem&, const IFileViewTreeItem&)> SecondaryCompare;
	if (!SecondarySortedColumn.IsNone())
	{
		SecondaryCompare = SourceControlFileViewColumn::GetColumnComparer(SecondarySortedColumn, PathFlags);
	}

	if (PrimarySortMode == EColumnSortMode::Ascending)
	{
		// NOTE: StableSort() would give a better experience when the sorted columns(s) has the same values and new values gets added, but it is slower
		//       with large changelists (7600 items was about 1.8x slower in average measured with Unreal Insight). Because this code runs in the main
		//       thread and can be invoked a lot, the trade off went if favor of speed.
		FileListNodes.Sort([this, &PrimaryCompare, &SecondaryCompare](const TSharedPtr<IChangelistTreeItem>& Lhs, const TSharedPtr<IChangelistTreeItem>& Rhs)
		{
			int32 Result = PrimaryCompare(static_cast<IFileViewTreeItem&>(*Lhs), static_cast<IFileViewTreeItem&>(*Rhs));
			if (Result < 0)
			{
				return true;
			}
			else if (Result > 0 || !SecondaryCompare)
			{
				return false;
			}
			else if (SecondarySortMode == EColumnSortMode::Ascending)
			{
				return SecondaryCompare(static_cast<IFileViewTreeItem&>(*Lhs), static_cast<IFileViewTreeItem&>(*Rhs)) < 0;
			}
			else
			{
				return SecondaryCompare(static_cast<IFileViewTreeItem&>(*Lhs), static_cast<IFileViewTreeItem&>(*Rhs)) > 0;
			}
		});
	}
	else
	{
		FileListNodes.Sort([this, &PrimaryCompare, &SecondaryCompare](const TSharedPtr<IChangelistTreeItem>& Lhs, const TSharedPtr<IChangelistTreeItem>& Rhs)
		{
			int32 Result = PrimaryCompare(static_cast<IFileViewTreeItem&>(*Lhs), static_cast<IFileViewTreeItem&>(*Rhs));
			if (Result > 0)
			{
				return true;
			}
			else if (Result < 0 || !SecondaryCompare)
			{
				return false;
			}
			else if (SecondarySortMode == EColumnSortMode::Ascending)
			{
				return SecondaryCompare(static_cast<IFileViewTreeItem&>(*Lhs), static_cast<IFileViewTreeItem&>(*Rhs)) < 0;
			}
			else
			{
				return SecondaryCompare(static_cast<IFileViewTreeItem&>(*Lhs), static_cast<IFileViewTreeItem&>(*Rhs)) > 0;
			}
		});
	}
}

void SSourceControlChangelistsWidget::OnChangelistSearchTextChanged(const FText& InFilterText)
{
	ChangelistTextFilter->SetRawFilterText(InFilterText);
	ChangelistExpandableArea->GetSearchBox()->SetError(ChangelistTextFilter->GetFilterErrorText());
}

void SSourceControlChangelistsWidget::OnUncontrolledChangelistSearchTextChanged(const FText& InFilterText)
{
	UncontrolledChangelistTextFilter->SetRawFilterText(InFilterText);
	UncontrolledChangelistExpandableArea->GetSearchBox()->SetError(UncontrolledChangelistTextFilter->GetFilterErrorText());
}

void SSourceControlChangelistsWidget::OnFileSearchTextChanged(const FText& InFilterText)
{
	FileTextFilter->SetRawFilterText(InFilterText);
	FileSearchBox->SetError(FileTextFilter->GetFilterErrorText());
}

void SSourceControlChangelistsWidget::PopulateItemSearchStrings(const IChangelistTreeItem& Item, TArray<FString>& OutStrings)
{
	switch (Item.GetTreeItemType())
	{
	case IChangelistTreeItem::Changelist:
		SChangelistTableRow::PopulateSearchString(static_cast<const FChangelistTreeItem&>(Item), OutStrings);
		break;

	case IChangelistTreeItem::ShelvedChangelist:
		SShelvedFilesTableRow::PopulateSearchString(static_cast<const FShelvedChangelistTreeItem&>(Item), OutStrings);
		break;

	case IChangelistTreeItem::UncontrolledChangelist:
		SUncontrolledChangelistTableRow::PopulateSearchString(static_cast<const FUncontrolledChangelistTreeItem&>(Item), OutStrings);
		break;

	case IChangelistTreeItem::File:
	case IChangelistTreeItem::ShelvedFile:
		SFileTableRow::PopulateSearchString(
			static_cast<const FFileTreeItem&>(Item),
			bShowingContentVersePath ? SourceControlFileViewColumn::EPathFlags::ShowingVersePath : SourceControlFileViewColumn::EPathFlags::Default,
			OutStrings);
		break;

	case IChangelistTreeItem::OfflineFile:
		SOfflineFileTableRow::PopulateSearchString(
			static_cast<const FOfflineFileTreeItem&>(Item),
			bShowingContentVersePath ? SourceControlFileViewColumn::EPathFlags::ShowingVersePath : SourceControlFileViewColumn::EPathFlags::Default,
			OutStrings);
		break;

	default:
		checkNoEntry();
	}
}

void SSourceControlChangelistsWidget::OnUnsavedAssetChanged(const FString& /*Filepath*/)
{
	OnRefreshUI(ERefreshFlags::UnsavedAssets);
}

void SSourceControlChangelistsWidget::OnPackageSaved(const FString& PackageFilename, UPackage* Package, FObjectPostSaveContext ObjectSaveContext)
{
	OutdatedTimestampFiles.Emplace(PackageFilename); // The background task will convert relative to absolute path.
}

void SSourceControlChangelistsWidget::OnPluginEdited(IPlugin& InPlugin)
{
	// The plugin's Verse path may have changed.  Recompute all Verse paths.
	bShouldRefreshVersePaths = true;
}

SListView<FChangelistTreeItemPtr>& SSourceControlChangelistsWidget::GetActiveFileListView() const
{
	return *StaticCastSharedPtr<SListView<FChangelistTreeItemPtr>>(FileListViewSwitcher->GetActiveWidget());
}

#undef LOCTEXT_NAMESPACE
