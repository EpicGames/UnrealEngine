// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/AsyncWork.h"
#include "CoreMinimal.h"

#include <atomic>
#include "ISourceControlProvider.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/TextFilter.h"
#include "SSourceControlCommon.h"
#include "Stats/Stats.h"
#include "UObject/ObjectSaveContext.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STreeView.h"

class FChangelistGroupTreeItem;
class IPlugin;
class SExpandableArea;
class SExpandableChangelistArea;
class SWidgetSwitcher;
class SSearchBox;
class USourceControlSettings;

/**
 * Displays the user source control change lists.
 */
class SSourceControlChangelistsWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSourceControlChangelistsWidget) {}
	SLATE_END_ARGS()

	/** Constructs the widget */
	void Construct(const FArguments& InArgs);
	virtual ~SSourceControlChangelistsWidget();

	/** Set selected files */
	void SetSelectedFiles(const TArray<FString>& Filenames);

private:
	enum class ERefreshFlags
	{
		SourceControlChangelists = 1 << 0,
		UnsavedAssets            = 1 << 1,
		UncontrolledChangelists  = 1 << 2,
		All = SourceControlChangelists | UnsavedAssets | UncontrolledChangelists,
	};
	FRIEND_ENUM_CLASS_FLAGS(ERefreshFlags)

	/** Queries files timestamp asynchronously. */
	class FAsyncTimestampUpdater
	{
	public:
		void DoWork();
		bool CanAbandon() { return true; }
		void Abandon() { bAbandon = true; }
		TStatId GetStatId() const { return TStatId(); }
		static const TCHAR* Name() { return TEXT("SourceControlChangelistsTimestampUpdater"); }
	public:
		std::atomic<bool> bAbandon;
		TSet<FString> RequestedFileTimestamps;
		TMap<FString, FDateTime> QueriedFileTimestamps;
	};

private:
	TSharedRef<SWidget> MakeToolBar();
	TSharedRef<STreeView<FChangelistTreeItemPtr>> CreateChangelistTreeView(TArray<TSharedPtr<IChangelistTreeItem>>& ItemSources);
	TSharedRef<SListView<FChangelistTreeItemPtr>> CreateChangelistFilesView();
	TSharedRef<SListView<FChangelistTreeItemPtr>> CreateUnsavedAssetsFilesView();

	TSharedRef<ITableRow> OnGenerateRow(FChangelistTreeItemPtr InTreeItem, const TSharedRef<STableViewBase>& OwnerTable);
	void OnGetChangelistChildren(FChangelistTreeItemPtr InParent, TArray<FChangelistTreeItemPtr>& OutChildren);
	void OnFileViewHiddenColumnsListChanged();

	EColumnSortPriority::Type GetColumnSortPriority(const FName ColumnId) const;
	EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;
	void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);
	bool IsFileViewSortedByFileStatusIcon() const;
	bool IsFileViewSortedByFileShelveIcon() const;
	bool IsFileViewSortedByLastModifiedTimestamp() const;
	void SortFileView();

	void OnChangelistSearchTextChanged(const FText& InFilterText);
	void OnUncontrolledChangelistSearchTextChanged(const FText& InFilterText);
	void OnFileSearchTextChanged(const FText& InFilterText);
	void PopulateItemSearchStrings(const IChangelistTreeItem& Item, TArray<FString>& OutStrings);
	void OnUnsavedAssetChanged(const FString& Filepath);

	FReply OnFilesDragged(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent);
	FReply OnUnsavedAssetsDragged(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent);

	void RequestChangelistsRefresh();
	void RequestFileStatusRefresh(const IChangelistTreeItem& Changelist);
	void RequestFileStatusRefresh(TSet<FString>&& Pathnames);
	void OnRefreshUI(ERefreshFlags RefreshFlags);
	void OnRefreshUnsavedAssetsWidgets(int64 CurrUpdateNum, const TFunction<void(TSharedPtr<IFileViewTreeItem>&)>& AddItemToFileView);
	void OnRefreshSourceControlWidgets(int64 CurreUpdateNum, const TFunction<void(TSharedPtr<IFileViewTreeItem>&)>& AddItemToFileView);
	void OnRefreshUncontrolledChangelistWidgets(int64 CurreUpdateNum, const TFunction<void(TSharedPtr<IFileViewTreeItem>&)>& AddItemToFileView);
	void ClearChangelistsTree();

	TSharedPtr<SWidget> OnOpenContextMenu();

	/** Returns the currently selected changelist state ptr or null in invalid cases */
	FSourceControlChangelistStatePtr GetCurrentChangelistState() const;
	FUncontrolledChangelistStatePtr GetCurrentUncontrolledChangelistState() const;
	FSourceControlChangelistPtr GetCurrentChangelist();
	TOptional<FUncontrolledChangelist> GetCurrentUncontrolledChangelist() const;
	FSourceControlChangelistStatePtr GetChangelistStateFromSelection();
	FSourceControlChangelistPtr GetChangelistFromSelection();

	/** Returns list of currently selected files */
	TArray<FString> GetSelectedFiles();
	bool HasFilesSelected() const;

	/** Returns list of currently selected shelved files */
	TArray<FString> GetSelectedShelvedFiles();
	bool HasShelvedFilesSelected() const;

	/**
	 * Splits selected files between Controlled and Uncontrolled files.
	 * @param 	OutControlledFiles 		Selected source controlled files will be added to this array.
	 * @param 	OutUncontrolledFiles	Selected uncontrolled files will be added to this array.
	 */
	void GetSelectedFiles(TArray<FString>& OutControlledFiles, TArray<FString>& OutUncontrolledFiles);

	/**
	 * Splits selected files between Controlled and Uncontrolled files.
	 * @param 	OutControlledFileStates 	Selected source controlled file states will be added to this array.
	 * @param 	OutUncontrolledFileStates	Selected uncontrolled file states will be added to this array.
	 */
	void GetSelectedFileStates(TArray<FSourceControlStateRef>& OutControlledFileStates, TArray<FSourceControlStateRef>& OutUncontrolledFileStates);

	/** Intercept Enter and Delete key presses to Submit or Delete the selected changelist (if conditions are met) */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	/** Changelist operations */
	void OnNewChangelist();
	void OnDeleteChangelist();
	bool CanDeleteChangelist() const;
	bool CanDeleteChangelist(FText* OutFailureMessage) const;
	void OnEditChangelist();
	void OnSubmitChangelist();
	FText GetCanSubmitChangelistToolTip() const;
	bool CanSubmitChangelist() const;
	bool CanSubmitChangelist(FText* OutFailureMessage) const;
	void OnValidateChangelist();
	bool CanValidateChangelist() const;

	/** Uncontrolled Changelist operations */
	void OnNewUncontrolledChangelist();
	void OnEditUncontrolledChangelist();
	bool CanEditUncontrolledChangelist();
	void OnDeleteUncontrolledChangelist();
	bool CanDeleteUncontrolledChangelist();

	/** Changelist & File operations */
	void OnRevertUnchanged();
	bool CanRevertUnchanged();
	void OnRevert();
	bool CanRevert();
	void OnShelve();

	/** Changelist & shelved files operations */
	void OnUnshelve();
	void OnDeleteShelvedFiles();

	/** Files operations */
	TValueOrError<void, void> TryMoveFiles();
	void OnShowHistory();
	void OnDiffAgainstDepot();
	bool CanDiffAgainstDepot();

	/** Shelved files operations */
	void OnDiffAgainstWorkspace();
	bool CanDiffAgainstWorkspace();

	/** Save assets in changelist operations */
	void OnSaveAssetsInChangelist();
	bool CanSaveAssetsInChangelist();

	/** Uncontrolled changelist module callback. */
	void OnUncontrolledChangelistStateChanged();

	/** Source control callbacks */
	void OnSourceControlProviderChanged(ISourceControlProvider& OldProvider, ISourceControlProvider& NewProvider);
	void OnSourceControlStateChanged();
	void OnItemDoubleClicked(TSharedPtr<IChangelistTreeItem> Item);
	void OnChangelistSelectionChanged(TSharedPtr<IChangelistTreeItem> SelectedItem, ESelectInfo::Type SelectionType);
	void OnChangelistsStatusUpdated(const TSharedRef<ISourceControlOperation>& InOperation, ECommandResult::Type InType);

	void OnStartSourceControlOperation(TSharedRef<ISourceControlOperation> Operation, const FText& Message);
	void OnEndSourceControlOperation(const TSharedRef<ISourceControlOperation>& Operation, ECommandResult::Type InType);

	// Wrapper functions on top of the source control ones to display slow tasks for synchronous operations or toast notifications for async ones.
	void Execute(const FText& Message, const TSharedRef<ISourceControlOperation>& InOperation, TSharedPtr<ISourceControlChangelist> InChangelist, const TArray<FString>& InFiles, EConcurrency::Type InConcurrency, const FSourceControlOperationComplete& InOperationCompleteDelegate);
	void Execute(const FText& Message, const TSharedRef<ISourceControlOperation>& InOperation, TSharedPtr<ISourceControlChangelist> InChangelist, const EConcurrency::Type InConcurrency, const FSourceControlOperationComplete& InOperationCompleteDelegate);
	void Execute(const FText& Message, const TSharedRef<ISourceControlOperation>& InOperation, const EConcurrency::Type InConcurrency, const FSourceControlOperationComplete& InOperationCompleteDelegate);
	void Execute(const FText& Message, const TSharedRef<ISourceControlOperation>& InOperation, const TArray<FString>& InFiles, EConcurrency::Type InConcurrency, const FSourceControlOperationComplete& InOperationCompleteDelegate);
	void ExecuteUncontrolledChangelistOperation(const FText& Message, const TFunction<void()>& UncontrolledChangelistTask);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/** Executes an operation to updates the changelist description of the provided changelist with a new description. */
	void EditChangelistDescription(const FText& InNewChangelistDescription, const FSourceControlChangelistStatePtr& InChangelistState);

	/** Invoked when a package is saved, to refresh the last saved timestamp. */
	void OnPackageSaved(const FString& PackageFilename, UPackage* Package, FObjectPostSaveContext ObjectSaveContext);

	void OnPluginEdited(IPlugin& InPlugin);

private:
	TSharedPtr<SExpandableChangelistArea> UnsavedAssetsExpandableArea;
	TSharedPtr<STreeView<FChangelistTreeItemPtr>> UnsavedAssetsTreeView;
	TArray<TSharedPtr<IChangelistTreeItem>> UnsavedAssetsTreeNode;
	
	TSharedPtr<SExpandableChangelistArea> ChangelistExpandableArea;
	TSharedPtr<STreeView<FChangelistTreeItemPtr>> ChangelistTreeView;
	TArray<TSharedPtr<IChangelistTreeItem>> ChangelistTreeNodes;

	TSharedPtr<SExpandableChangelistArea> UncontrolledChangelistExpandableArea;
	TSharedPtr<STreeView<FChangelistTreeItemPtr>> UncontrolledChangelistTreeView;
	TArray<TSharedPtr<IChangelistTreeItem>> UncontrolledChangelistTreeNodes;
	
	TSharedPtr<SListView<FChangelistTreeItemPtr>> FileListView;
	TArray<TSharedPtr<IChangelistTreeItem>> FileListNodes;
	TArray<FName> FileViewHiddenColumnsList;

	TSharedPtr<SListView<FChangelistTreeItemPtr>> UnsavedAssetsFileListView;
	TSharedPtr<SWidgetSwitcher> FileListViewSwitcher;
	
	SListView<FChangelistTreeItemPtr>& GetActiveFileListView() const;

	TMap<TSharedPtr<void>, TSharedPtr<IChangelistTreeItem>> SourceControlItemCache;
	TMap<TSharedPtr<void>, TSharedPtr<IChangelistTreeItem>> UncontrolledChangelistItemCache;
	TMap<FString, TSharedPtr<IFileViewTreeItem>> OfflineFileItemCache;

	TUniquePtr<FAsyncTask<FAsyncTimestampUpdater>> TimestampUpdateTask;
	TSet<FString> OutdatedTimestampFiles;

	/** Source control state changed delegate handle */
	FDelegateHandle SourceControlStateChangedDelegateHandle;

	int64 UpdateRequestNum = 0;
	bool bInitialRefreshDone = false;
	bool bShouldRefresh = false;
	bool bShouldRefreshVersePaths = false;
	bool bShowingContentVersePath = false;
	bool bSourceControlAvailable = false;
	bool bUpdateMonitoredFileStatusList = false;

	/** Files to select after refresh */
	TArray<FString> FilesToSelect;

	FName PrimarySortedColumn;
	FName SecondarySortedColumn;
	EColumnSortMode::Type PrimarySortMode = EColumnSortMode::Ascending;
	EColumnSortMode::Type SecondarySortMode = EColumnSortMode::None;

	TSharedPtr<TTextFilter<const IChangelistTreeItem&>> ChangelistTextFilter;
	TSharedPtr<TTextFilter<const IChangelistTreeItem&>> UncontrolledChangelistTextFilter;
	TSharedPtr<TTextFilter<const IChangelistTreeItem&>> FileTextFilter;
	TSharedPtr<SSearchBox> FileSearchBox;

	void StartRefreshStatus();
	void TickRefreshStatus(double InDeltaTime);
	void EndRefreshStatus();

	FText RefreshStatus;
	bool bIsRefreshing = false;
	double RefreshStatusStartSecs;

	float ChangelistAreaSize = 0.3f; // [0.0f, 1.0f]
	float FileAreaSize = 0.7f;
};
