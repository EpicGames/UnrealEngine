// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSourceControlCheckedOutDialog.h"
#include "SSourceControlChangelistRows.h"
#include "AssetToolsModule.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "Styling/AppStyle.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/Paths.h"
#include "HAL/PlatformFileManager.h"
#include "Framework/Application/SlateApplication.h"
#include "SPrimaryButton.h"
#include "IAssetTools.h"
#include "ISourceControlModule.h"
#include "SourceControlAssetDataCache.h"
#include "Algo/Find.h"
#include "Algo/Count.h"
#include "ActorFolder.h"
#include "ActorFolderDesc.h"
#include "AssetDefinitionRegistry.h"

#define LOCTEXT_NAMESPACE "SSourceControlConflict"

void SSourceControlCheckedOutDialog::Construct(const FArguments& InArgs)
{
	ParentFrame = InArgs._ParentWindow;
	SortByColumn = SourceControlFileViewColumn::Name::Id();
	SortMode = EColumnSortMode::Ascending;
	bShowingContentVersePath = FAssetToolsModule::GetModule().Get().ShowingContentVersePath();

	ListViewItems.Reserve(InArgs._Items.Num());
	for (const auto & Item : InArgs._Items)
	{
		ListViewItems.Add(MakeShared<FFileTreeItem>(Item));
	}

	TSharedPtr<SHeaderRow> HeaderRowWidget;
	HeaderRowWidget = SNew(SHeaderRow);

	bool bShowColumnAssetName = InArgs._ShowColumnAssetName;
	bool bShowColumnAssetClass = InArgs._ShowColumnAssetClass;
	bool bShowColumnUserName = InArgs._ShowColumnUserName;

	if (bShowColumnUserName)
	{
		bool bAnyCheckedOut = false;
		for (const FChangelistTreeItemPtr& Item : ListViewItems)
		{
			if (!static_cast<IFileViewTreeItem&>(*Item).GetCheckedOutBy().IsEmpty())
			{
				bAnyCheckedOut = true;
				break;
			}
		}
		bShowColumnUserName = bAnyCheckedOut;
	}

	if (bShowColumnAssetName)
	{
		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(SourceControlFileViewColumn::Name::Id())
			.DefaultLabel(LOCTEXT("AssetColumnLabel", "Asset"))
			.DefaultTooltip(SourceControlFileViewColumn::Name::GetToolTipText())
			.SortMode(this, &SSourceControlCheckedOutDialog::GetColumnSortMode, SourceControlFileViewColumn::Name::Id())
			.OnSort(this, &SSourceControlCheckedOutDialog::OnColumnSortModeChanged)
			.FillWidth(1.5f)
		);

		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(SourceControlFileViewColumn::Path::Id())
			.DefaultLabel(LOCTEXT("FileColumnLabel", "File"))
			.DefaultTooltip(SourceControlFileViewColumn::Path::GetToolTipText())
			.SortMode(this, &SSourceControlCheckedOutDialog::GetColumnSortMode, SourceControlFileViewColumn::Path::Id())
			.OnSort(this, &SSourceControlCheckedOutDialog::OnColumnSortModeChanged)
			.FillWidth(3.0f)
		);
	}

	if (bShowColumnAssetClass)
	{
		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(SourceControlFileViewColumn::Type::Id())
			.DefaultLabel(SourceControlFileViewColumn::Type::GetDisplayText())
			.DefaultTooltip(SourceControlFileViewColumn::Type::GetToolTipText())
			.SortMode(this, &SSourceControlCheckedOutDialog::GetColumnSortMode, SourceControlFileViewColumn::Type::Id())
			.OnSort(this, &SSourceControlCheckedOutDialog::OnColumnSortModeChanged)
			.FillWidth(1.0f)
		);
	}

	if (bShowColumnUserName)
	{
		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(SourceControlFileViewColumn::CheckedOutByUser::Id())
			.DefaultLabel(SourceControlFileViewColumn::CheckedOutByUser::GetDisplayText())
			.DefaultTooltip(SourceControlFileViewColumn::CheckedOutByUser::GetToolTipText())
			.SortMode(this, &SSourceControlCheckedOutDialog::GetColumnSortMode, SourceControlFileViewColumn::CheckedOutByUser::Id())
			.OnSort(this, &SSourceControlCheckedOutDialog::OnColumnSortModeChanged)
			.FillWidth(1.0f)
		);
	}

	TSharedPtr<SHorizontalBox> ButtonsBox = SNew(SHorizontalBox);

	bool bShowCheckBox = !InArgs._CheckBoxText.IsEmpty();
	if (bShowCheckBox)
	{
		CheckBox = SNew(SCheckBox);
		CheckBox->SetIsChecked(ECheckBoxState::Checked);

		ButtonsBox->AddSlot()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(5, 0)
				[
					CheckBox.ToSharedRef()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(5, 5)
				[
					SNew(STextBlock)
					.Text(InArgs._CheckBoxText)
				]
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.Padding(5, 0)
			[
				SAssignNew(CloseButton, SPrimaryButton)
				.Text(InArgs._CloseText)
				.OnClicked(this, &SSourceControlCheckedOutDialog::CloseClicked)
			]
		];
	}
	else
	{
		ButtonsBox->AddSlot()
		.AutoWidth()
		.Padding(5, 0)
		[
			SAssignNew(CloseButton, SPrimaryButton)
			.Text(InArgs._CloseText)
			.OnClicked(this, &SSourceControlCheckedOutDialog::CloseClicked)
		];
	}

	TSharedPtr<SVerticalBox> Contents;

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Padding(16.f)
		[
			SAssignNew(Contents, SVerticalBox)
		]
	];

	Contents->AddSlot()
	.AutoHeight()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Left)
		.Padding(FMargin(0.0f, 0.0f, 16.0f, 0.0f))
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.WarningWithColor.Large"))
		]
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew(SRichTextBlock)
			.DecoratorStyleSet(&FAppStyle::Get())
			.Text(InArgs._MessageText)
			.AutoWrapText(true)
		]
	];

	Contents->AddSlot()
	.Padding(0.0f, 16.0f, 0.0f, 0.0f)
	[
		SNew(SBorder)
		.Visibility_Lambda([this]() -> EVisibility
		{
			return ListViewItems.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
		})
		[
			SNew(SBox)
			.HeightOverride(200.0f)
			.WidthOverride(800.0f)
			[
				SAssignNew(ListView, SListView<FChangelistTreeItemPtr>)
				.ListItemsSource(&ListViewItems)
				.OnGenerateRow(this, &SSourceControlCheckedOutDialog::OnGenerateRowForList)
				.HeaderRow(HeaderRowWidget)
				.SelectionMode(ESelectionMode::Single)
			]
		]
	];

	Contents->AddSlot()
	.AutoHeight()
	.Padding(0.0f, 16.0f, 0.0f, 0.0f)
	.HAlign(bShowCheckBox ? HAlign_Fill : HAlign_Right)
	.VAlign(VAlign_Bottom)
	[
		ButtonsBox.ToSharedRef()
	];

	RequestSort();
}

TSharedRef<ITableRow> SSourceControlCheckedOutDialog::OnGenerateRowForList(FChangelistTreeItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedRef<ITableRow> Row =
		SNew(SFileTableRow, OwnerTable)
		.TreeItemToVisualize(InItem)
		.PathFlags(bShowingContentVersePath ? SourceControlFileViewColumn::EPathFlags::ShowingVersePath : SourceControlFileViewColumn::EPathFlags::Default);

	return Row;
}

EColumnSortMode::Type SSourceControlCheckedOutDialog::GetColumnSortMode(const FName ColumnId) const
{
	if (SortByColumn != ColumnId)
	{
		return EColumnSortMode::None;
	}

	return SortMode;
}

void SSourceControlCheckedOutDialog::OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	SortByColumn = ColumnId;
	SortMode = InSortMode;

	RequestSort();
}

void SSourceControlCheckedOutDialog::RequestSort()
{
	SortTree();

	ListView->RequestListRefresh();
}

void SSourceControlCheckedOutDialog::SortTree()
{
	TFunction<bool(const IFileViewTreeItem&, const IFileViewTreeItem&)> SortPredicate = SourceControlFileViewColumn::GetSortPredicate(
		SortMode, SortByColumn, bShowingContentVersePath ? SourceControlFileViewColumn::EPathFlags::ShowingVersePath : SourceControlFileViewColumn::EPathFlags::Default);
	if (SortPredicate)
	{
		Algo::SortBy(
			ListViewItems,
			[](const FChangelistTreeItemPtr& ListViewItem) -> const IFileViewTreeItem&
			{
				return static_cast<IFileViewTreeItem&>(*ListViewItem);
			},
			SortPredicate);
	}
}

FReply SSourceControlCheckedOutDialog::CloseClicked()
{
	ParentFrame.Pin()->RequestDestroyWindow();

	return FReply::Handled();
}

FReply SSourceControlCheckedOutDialog::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if ((InKeyEvent.GetKey() == EKeys::Escape && CloseButton.IsValid()))
	{
		return CloseClicked();
	}

	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
