// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Input/SCheckBox.h"
#include "SSourceControlCommon.h"

class SWindow;

class SSourceControlCheckedOutDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSourceControlCheckedOutDialog)
		: _ParentWindow()
		, _Items()
		, _ShowColumnAssetName(true)
		, _ShowColumnAssetClass(true)
		, _ShowColumnUserName(true)
		, _MessageText()
		, _CloseText()
		, _CheckBoxText()
		{ }

		SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)
		SLATE_ARGUMENT(TArray<FSourceControlStateRef>, Items)
		SLATE_ARGUMENT(bool, ShowColumnAssetName)
		SLATE_ARGUMENT(bool, ShowColumnAssetClass)
		SLATE_ARGUMENT(bool, ShowColumnUserName)
		SLATE_ARGUMENT(FText, MessageText)
		SLATE_ARGUMENT(FText, CloseText)
		SLATE_ARGUMENT(FText, CheckBoxText)

	SLATE_END_ARGS()

	/** Constructs the widget */
	void Construct(const FArguments& InArgs);

	/** Get whether to hide the checkbox is checked */
	bool IsCheckBoxChecked() { return CheckBox.IsValid() ? CheckBox->IsChecked() : false; }

private:
	/** Callback to generate ListBoxRows */
	TSharedRef<ITableRow> OnGenerateRowForList(FChangelistTreeItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/**
	 * Returns the current column sort mode (ascending or descending) if the ColumnId parameter matches the current
	 * column to be sorted by, otherwise returns EColumnSortMode_None.
	 *
	 * @param	ColumnId	Column ID to query sort mode for.
	 * @return	The sort mode for the column, or EColumnSortMode_None if it is not known.
	 */
	EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;

	/**
	 * Callback for SHeaderRow::Column::OnSort, called when the column to sort by is changed.
	 *
	 * @param	ColumnId	The new column to sort by
	 * @param	InSortMode	The sort mode (ascending or descending)
	 */
	void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);

	/**
	 * Requests that the source list data be sorted according to the current sort column and mode,
	 * and refreshes the list view.
	 */
	void RequestSort();

	/**
	 * Sorts the source list data according to the current sort column and mode.
	 */
	void SortTree();

	/** Called when the cancel button is clicked */
	FReply CloseClicked();

private:
	//~ Begin SWidget Interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget Interface

	/** Pointer to the parent modal window */
	TWeakPtr<SWindow> ParentFrame;

	/** Collection of objects to display in the List View */
	TArray<FChangelistTreeItemPtr> ListViewItems;

	/** ListBox for displaying items */
	TSharedPtr<SListView<FChangelistTreeItemPtr>> ListView;

	/** Specify which column to sort with */
	FName SortByColumn;

	/** Currently selected sorting mode */
	EColumnSortMode::Type SortMode = EColumnSortMode::Ascending;

	bool bShowingContentVersePath = false;

	/** The close button widget. */
	TSharedPtr<SButton> CloseButton;

	/** The checkbox widget. */
	TSharedPtr<SCheckBox> CheckBox;
};
