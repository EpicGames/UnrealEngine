// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSourceControlCommon.h"

#include "Algo/Count.h"
#include "Algo/Find.h"
#include "Algo/Replace.h"
#include "AssetRegistry/AssetData.h"
#include "ActorFolder.h"
#include "ActorFolderDesc.h"
#include "AssetDefinitionRegistry.h"
#include "AssetToolsModule.h"
#include "Styling/AppStyle.h"
#include "RevisionControlStyle/RevisionControlStyle.h"
#include "IAssetTools.h"
#include "ISourceControlModule.h"
#include "SourceControlAssetDataCache.h"
#include "SourceControlHelpers.h"
#include "SSourceControlFileDialog.h"

#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Layout/SBox.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Logging/MessageLog.h"
#include "Misc/PathViews.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "SourceControlChangelist"

//////////////////////////////////////////////////////////////////////////

FChangelistTreeItemPtr IChangelistTreeItem::GetParent() const
{
	return Parent;
}

const TArray<FChangelistTreeItemPtr>& IChangelistTreeItem::GetChildren() const
{
	return Children;
}

void IChangelistTreeItem::AddChild(TSharedRef<IChangelistTreeItem> Child)
{
	Child->Parent = AsShared();
	Children.Add(MoveTemp(Child));
}

void IChangelistTreeItem::RemoveChild(const TSharedRef<IChangelistTreeItem>& Child)
{
	if (Children.Remove(Child))
	{
		Child->Parent = nullptr;
	}
}

void IChangelistTreeItem::RemoveAllChildren()
{
	for (TSharedPtr<IChangelistTreeItem>& Child : Children)
	{
		Child->Parent = nullptr;
	}
	Children.Reset();
}

namespace SSourceControlCommonPrivate
{
static FString RetrieveAssetName(const FAssetData& InAssetData)
{
	static const FName NAME_ActorLabel(TEXT("ActorLabel"));

	if (InAssetData.FindTag(NAME_ActorLabel))
	{
		FString ResultAssetName;
		InAssetData.GetTagValue(NAME_ActorLabel, ResultAssetName);
		return ResultAssetName;
	}
	else if (InAssetData.FindTag(FPrimaryAssetId::PrimaryAssetDisplayNameTag))
	{
		FString ResultAssetName;
		InAssetData.GetTagValue(FPrimaryAssetId::PrimaryAssetDisplayNameTag, ResultAssetName);
		return ResultAssetName;
	}
	else if (InAssetData.AssetClassPath == UActorFolder::StaticClass()->GetClassPathName())
	{
		FString ActorFolderPath = UActorFolder::GetAssetRegistryInfoFromPackage(InAssetData.PackageName).GetDisplayName();
		if (!ActorFolderPath.IsEmpty())
		{
			return ActorFolderPath;
		}
	}

	return InAssetData.AssetName.ToString();
}

static FString RetrieveAssetPath(const FAssetData& InAssetData)
{
	int32 LastDot = -1;
	FString Path = InAssetData.GetObjectPathString();

	// Strip asset name from object path
	if (Path.FindLastChar('.', LastDot))
	{
		Path.LeftInline(LastDot);
	}

	return Path;
}

static FString RetrieveAssetTypeName(const FAssetData& InAssetData)
{
	if (UAssetDefinitionRegistry* AssetDefinitionRegistry = UAssetDefinitionRegistry::Get())
	{
		const UAssetDefinition* AssetDefinition = AssetDefinitionRegistry->GetAssetDefinitionForAsset(InAssetData);
		if (AssetDefinition)
		{
			return AssetDefinition->GetAssetDisplayName().ToString();
		}
	}

	return InAssetData.AssetClassPath.ToString();
}

static const FAssetData* GetUserFacingAsset(const TArray<FAssetData>* Assets, int32& OutNumUserFacingAssets)
{
	if (Assets == nullptr || Assets->IsEmpty())
	{
		OutNumUserFacingAssets = 0;
		return nullptr;
	}

	auto IsNotRedirector = [](const FAssetData& InAssetData) { return !InAssetData.IsRedirector(); };
	OutNumUserFacingAssets = Algo::CountIf(*Assets, IsNotRedirector);

	if (OutNumUserFacingAssets == 1)
	{
		return Algo::FindByPredicate(*Assets, IsNotRedirector);
	}
	else
	{
		return Assets->GetData();
	}
}

static bool RefreshAssetVersePathInternal(const TArray<FAssetData>* Assets, UE::Core::FVersePath& InOutAssetVersePath)
{
	UE::Core::FVersePath TempAssetVersePath;

	{
		int32 NumUserFacingAsset = 0;
		const FAssetData* AssetData = GetUserFacingAsset(Assets, NumUserFacingAsset);
		if (AssetData != nullptr && NumUserFacingAsset == 1)
		{
			TempAssetVersePath = AssetData->GetVersePath();
		}
	}

	if (TempAssetVersePath != InOutAssetVersePath)
	{
		InOutAssetVersePath = MoveTemp(TempAssetVersePath);
		return true;
	}

	return false;
}

static void RefreshAssetInformationInternal(const TArray<FAssetData>* Assets, const FString& InFilename, FString& OutAssetName, FString& OutAssetPath, UE::Core::FVersePath& OutAssetVersePath, FString& OutAssetType, FString& OutAssetTypeName, FColor& OutAssetTypeColor)
{
	// Initialize display-related members
	FString TempAssetName = SSourceControlCommon::GetDefaultAssetName().ToString();
	FString TempAssetPath;
	UE::Core::FVersePath TempAssetVersePath;
	FString TempAssetType = SSourceControlCommon::GetDefaultAssetType().ToString();
	FString TempAssetTypeName = SSourceControlCommon::GetDefaultAssetType().ToString();
	FColor TempAssetColor = FColor(		// Copied from ContentBrowserCLR.cpp
		127 + FColor::Red.R / 2,	// Desaturate the colors a bit (GB colors were too.. much)
		127 + FColor::Red.G / 2,
		127 + FColor::Red.B / 2,
		200); // Opacity

	const FString Extension = FPaths::GetExtension(InFilename);
	bool bIsPackageExtension =
		FPackageName::IsPackageExtension(*Extension) ||
		FPackageName::IsVerseExtension(*Extension);

	int32 NumUserFacingAsset = 0;
	const FAssetData* AssetData = GetUserFacingAsset(Assets, NumUserFacingAsset);
	if (AssetData != nullptr)
	{
		FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();

		TempAssetName = RetrieveAssetName(*AssetData);
		TempAssetPath = RetrieveAssetPath(*AssetData);
		TempAssetTypeName = RetrieveAssetTypeName(*AssetData);
		TempAssetColor = FColor::White;

		if (NumUserFacingAsset > 1)
		{
			TempAssetType = SSourceControlCommon::GetDefaultMultipleAsset().ToString();

			for (const FAssetData& OtherAssetData : *Assets)
			{
				if (AssetData != &OtherAssetData)
				{
					TempAssetName += TEXT(";") + RetrieveAssetName(OtherAssetData);
				}
			}
		}
		else
		{
			TempAssetType = AssetData->AssetClassPath.ToString();

			if (AssetToolsModule.Get().GetOnShowingContentVersePath().IsBound())
			{
				TempAssetVersePath = AssetData->GetVersePath();
			}

			const TSharedPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(AssetData->GetClass()).Pin();

			if (AssetTypeActions.IsValid())
			{
				TempAssetColor = AssetTypeActions->GetTypeColor();
			}
		}
	}
	else
	{
		FString PackageName;
		if (bIsPackageExtension && FPackageName::TryConvertFilenameToLongPackageName(InFilename, PackageName))
		{
			// Fake asset name, asset path from the package name
			TempAssetName = FPackageName::GetShortName(PackageName);
			TempAssetPath = MoveTemp(PackageName);
		}
		else
		{
			TempAssetName = FPaths::GetCleanFilename(InFilename);
			TempAssetPath = InFilename;
			TempAssetType = FText::Format(SSourceControlCommon::GetDefaultUnknownAssetType(), FText::FromString(Extension.ToUpper())).ToString();
			TempAssetTypeName = TempAssetType;

			FPaths::MakePlatformFilename(TempAssetPath);
		}
	}

	// Finally, assign the temp variables to the member variables
	OutAssetName = MoveTemp(TempAssetName);
	OutAssetPath = MoveTemp(TempAssetPath);
	OutAssetVersePath = MoveTemp(TempAssetVersePath);
	OutAssetType = MoveTemp(TempAssetType);
	OutAssetTypeName = MoveTemp(TempAssetTypeName);
	OutAssetTypeColor = TempAssetColor;
}
}

//////////////////////////////////////////////////////////////////////////

const FString IFileViewTreeItem::DefaultStrValue; // Default is an empty string.

void IFileViewTreeItem::SetLastModifiedDateTime(const FDateTime& Timestamp)
{
	if (Timestamp != LastModifiedDateTime) // Pay the text conversion only if needed.
	{
		LastModifiedDateTime = Timestamp;
		if (Timestamp != FDateTime::MinValue())
		{
			LastModifiedTimestampText = FText::AsDateTime(Timestamp, EDateTimeStyle::Short);
		}
		else
		{
			LastModifiedTimestampText = FText::GetEmpty();
		}
	}
}

//////////////////////////////////////////////////////////////////////////

FString FUnsavedAssetsTreeItem::GetDisplayString() const
{
	return "";
}

FFileTreeItem::FFileTreeItem(FSourceControlStateRef InFileState, bool bBeautifyPaths, bool bIsShelvedFile)
	: IFileViewTreeItem(bIsShelvedFile ? IChangelistTreeItem::ShelvedFile : IChangelistTreeItem::File)
	, FileState(InFileState)
	, CheckBoxState(ECheckBoxState::Checked)
	, MinTimeBetweenUpdate(FTimespan::FromSeconds(5.f))
	, LastUpdateTime()
	, bAssetsUpToDate(false)
{
	// Initialize asset data first

	if (bBeautifyPaths)
	{
		FSourceControlAssetDataCache& AssetDataCache = ISourceControlModule::Get().GetAssetDataCache();
		bAssetsUpToDate = AssetDataCache.GetAssetDataArray(FileState, Assets);
	}
	else
	{
		// We do not need to wait for AssetData from the cache.
		bAssetsUpToDate = true;
	}

	RefreshAssetInformation();
}

int32 FFileTreeItem::GetIconSortingPriority() const
{
	if (!FileState->IsCurrent())        { return 0; } // First if sorted in ascending order.
	if (FileState->IsUnknown())         { return 1; }
	if (FileState->IsConflicted())      { return 2; }
	if (FileState->IsCheckedOutOther()) { return 3; }
	if (FileState->IsCheckedOut())      { return 4; }
	if (FileState->IsDeleted())         { return 5; }
	if (FileState->IsAdded())           { return 6; }
	else                                { return 7; }
}

const FString& FFileTreeItem::GetCheckedOutBy() const
{
	CheckedOutBy.Reset();
	FileState->IsCheckedOutOther(&CheckedOutBy);
	return CheckedOutBy;
}

FText FFileTreeItem::GetCheckedOutByUser() const
{
	return FText::FromString(GetCheckedOutBy());
}

FText FFileTreeItem::GetFileName() const
{
	FString Filename = FileState->GetFilename();
	FPaths::MakePlatformFilename(Filename);
	return FText::FromString(MoveTemp(Filename));
}

void FFileTreeItem::RefreshAssetInformation()
{
	// Initialize display-related members
	SSourceControlCommonPrivate::RefreshAssetInformationInternal(Assets.Get(), FileState->GetFilename(), AssetNameStr, AssetPathStr, AssetVersePathStruct, AssetTypeStr, AssetTypeNameStr, AssetTypeColor);
	AssetName = FText::FromString(AssetNameStr);
	AssetPath = FText::FromString(AssetPathStr);
	AssetVersePath = FText::FromString(AssetVersePathStruct.ToString());
	AssetType = FText::FromString(AssetTypeStr);
	AssetTypeName = FText::FromString(AssetTypeNameStr);
}

bool FFileTreeItem::RefreshVersePath()
{
	return SSourceControlCommonPrivate::RefreshAssetVersePathInternal(Assets.Get(), AssetVersePathStruct);
}

FText FFileTreeItem::GetAssetName() const
{
	return AssetName;
}

FText FFileTreeItem::GetAssetName()
{
	const FTimespan CurrentTime = FTimespan::FromSeconds(FPlatformTime::Seconds());

	if ((!bAssetsUpToDate) && ((CurrentTime - LastUpdateTime) > MinTimeBetweenUpdate))
	{
		FSourceControlAssetDataCache& AssetDataCache = ISourceControlModule::Get().GetAssetDataCache();
		LastUpdateTime = CurrentTime;

		if (AssetDataCache.GetAssetDataArray(FileState, Assets))
		{
			bAssetsUpToDate = true;
			RefreshAssetInformation();
		}
	}

	return AssetName;
}

//////////////////////////////////////////////////////////////////////////

FText FShelvedChangelistTreeItem::GetDisplayText() const
{
	return LOCTEXT("SourceControl_ShelvedFiles", "Shelved Items");
}

//////////////////////////////////////////////////////////////////////////

FOfflineFileTreeItem::FOfflineFileTreeItem(const FString& InFilename)
	: IFileViewTreeItem(IChangelistTreeItem::OfflineFile)
	, CheckBoxState(ECheckBoxState::Checked)
	, FilenameStr(InFilename)
{
	{
		FString TempFilename = FilenameStr;
		FPaths::MakePlatformFilename(TempFilename);
		Filename = FText::FromString(MoveTemp(TempFilename));
	}

	USourceControlHelpers::GetAssetData(InFilename, Assets);

	RefreshAssetInformation();
}

void FOfflineFileTreeItem::RefreshAssetInformation()
{
	SSourceControlCommonPrivate::RefreshAssetInformationInternal(&Assets, FilenameStr, AssetNameStr, AssetPathStr, AssetVersePathStruct, AssetTypeStr, AssetTypeNameStr, AssetTypeColor);
	AssetName = FText::FromString(AssetNameStr);
	AssetPath = FText::FromString(AssetPathStr);
	AssetVersePath = FText::FromString(AssetVersePathStruct.ToString());
	AssetType = FText::FromString(AssetTypeStr);
	AssetTypeName = FText::FromString(AssetTypeNameStr);
}

bool FOfflineFileTreeItem::RefreshVersePath()
{
	return SSourceControlCommonPrivate::RefreshAssetVersePathInternal(&Assets, AssetVersePathStruct);
}

//////////////////////////////////////////////////////////////////////////
namespace SSourceControlCommon
{

TSharedRef<SWidget> GetSCCStatusWidget(FSourceControlStateRef InFileState)
{
	const float SizeOverride = 16;

	return SNew(SOverlay)
		// Source control state
		+ SOverlay::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.WidthOverride(SizeOverride)
			.HeightOverride(SizeOverride)
			[
				SNew(SImage)
				.Image_Lambda(
					[InFileState]() -> const FSlateBrush*
					{
						return InFileState->GetIcon().GetIcon();
					}
				)
				.ToolTipText_Lambda(
					[InFileState]() -> FText
					{
						return InFileState->GetDisplayTooltip();
					}
				)
			]
		];
}

TSharedRef<SWidget> GetSCCStatusWidget()
{
	const float SizeOverride = 16;

	return SNew(SOverlay)
		// Source control state
		+ SOverlay::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Top)
			[
				SNew(SBox)
				.WidthOverride(SizeOverride)
				.HeightOverride(SizeOverride)
			];
}

TSharedRef<SWidget> GetSCCShelveWidget(bool bIsShelvedFile)
{
	if (bIsShelvedFile)
	{
		const FSlateBrush* IconBrush = FRevisionControlStyleManager::Get().GetBrush("RevisionControl.Shelved");

		return SNew(SOverlay)
			// Source control shelved
			+ SOverlay::Slot()
			[
				SNew(SImage)
				.Image(IconBrush)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.ToolTipText(LOCTEXT("SourceControl_Shelved", "Shelved"))
			];
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> GetSCCShelveWidget()
{
	return GetSCCShelveWidget(/*bIsShelvedFile=*/false);
}

FText GetDefaultAssetName()
{
	return LOCTEXT("SourceControl_DefaultAssetName", "Unavailable");
}

FText GetDefaultAssetType()
{
	return LOCTEXT("SourceControl_DefaultAssetType", "Unknown");
}

FText GetDefaultUnknownAssetType()
{
	return LOCTEXT("SourceControl_FileTypeDefault", "{0} File");
}

FText GetDefaultMultipleAsset()
{
	return LOCTEXT("SourceCOntrol_ManyAssetType", "Multiple Assets");
}

FText GetSingleLineChangelistDescription(const FText& InFullDescription, ESingleLineFlags Flags)
{
	FString DescriptionTextAsString = InFullDescription.ToString();
	DescriptionTextAsString.TrimStartAndEndInline();

	if ((Flags & ESingleLineFlags::Mask_NewlineBehavior) == ESingleLineFlags::NewlineConvertToSpace)
	{
		static constexpr TCHAR Replacer = TCHAR(' ');
		// Replace all non-space whitespace characters with space
		Algo::ReplaceIf(DescriptionTextAsString,
			[](TCHAR C) { return FChar::IsWhitespace(C) && C != Replacer; },
			Replacer);
	}
	else
	{
		int32 NewlineStartIndex = INDEX_NONE;
		DescriptionTextAsString.FindChar(TCHAR('\n'), NewlineStartIndex);
		if (NewlineStartIndex != INDEX_NONE)
		{
			DescriptionTextAsString.LeftInline(NewlineStartIndex);
		}

		// Trim any trailing carriage returns
		if (DescriptionTextAsString.EndsWith(TEXT("\r"), ESearchCase::CaseSensitive))
		{
			DescriptionTextAsString.LeftChopInline(1);
		}
	}

	return InFullDescription.IsCultureInvariant() ? FText::AsCultureInvariant(DescriptionTextAsString) : FText::FromString(DescriptionTextAsString);
}

/** Wraps the execution of a changelist operations with a slow task. */
void ExecuteChangelistOperationWithSlowTaskWrapper(const FText& Message, const TFunction<void()>& ChangelistTask)
{
	// NOTE: This is a ugly workaround for P4 because the generic popup feedback operations in FScopedSourceControlProgress() was suppressed for all synchrounous
	//       operations. For other source control providers, the popup still shows up and showing a slow task and the FScopedSourceControlProgress at the same
	//       time is a bad user experience. Until we fix source control popup situation in general in the Editor, this hack is in place to avoid the double popup.
	//       At the time of writing, the other source control provider that supports changelists is Plastic.
	if (ISourceControlModule::Get().GetProvider().GetName() == "Perforce")
	{
		FScopedSlowTask Progress(0.f, Message);
		Progress.MakeDialog();
		ChangelistTask();
	}
	else
	{
		ChangelistTask();
	}
}

/** Wraps the execution of an uncontrolled changelist operations with a slow task. */
void ExecuteUncontrolledChangelistOperationWithSlowTaskWrapper(const FText& Message, const TFunction<void()>& UncontrolledChangelistTask)
{
	ExecuteChangelistOperationWithSlowTaskWrapper(Message, UncontrolledChangelistTask);
}

TOptional<FNotificationInfo> ConstructSourceControlOperationNotification(const FText& Message)
{
	if (Message.IsEmpty())
	{
		return {};
	}

	FNotificationInfo NotificationInfo(Message);
	NotificationInfo.ExpireDuration = 6.0f;
	NotificationInfo.Hyperlink = FSimpleDelegate::CreateLambda([]() { FGlobalTabmanager::Get()->TryInvokeTab(FName("OutputLog")); });
	NotificationInfo.HyperlinkText = LOCTEXT("ShowOutputLogHyperlink", "Show Output Log");

	return NotificationInfo;
}

/** Displays toast notification to report the status of task. */
void DisplaySourceControlOperationNotification(const FText& Message, SNotificationItem::ECompletionState CompletionState)
{
	if (TOptional<FNotificationInfo> NotificationInfo = ConstructSourceControlOperationNotification(Message))
	{
		DisplaySourceControlOperationNotification(NotificationInfo.GetValue(), CompletionState);
	}
}

void DisplaySourceControlOperationNotification(const FNotificationInfo& NotificationInfo, SNotificationItem::ECompletionState CompletionState)
{
	if (!NotificationInfo.Text.IsSet())
	{
		return;
	}

	FMessageLog("SourceControl").Message(CompletionState == SNotificationItem::ECompletionState::CS_Fail ? EMessageSeverity::Error : EMessageSeverity::Info, NotificationInfo.Text.Get());

	FSlateNotificationManager::Get().AddNotification(NotificationInfo)->SetCompletionState(CompletionState);
}

bool OpenConflictDialog(const TArray<FSourceControlStateRef>& InFilesConflicts)
{
	TSharedPtr<SWindow> Window;
	TSharedPtr<SSourceControlFileDialog> SourceControlFileDialog;

	Window = SNew(SWindow)
			 .Title(LOCTEXT("CheckoutPackagesDialogTitle", "Check Out Assets"))
			 .SizingRule(ESizingRule::UserSized)
			 .ClientSize(FVector2D(1024.0f, 512.0f))
			 .SupportsMaximize(false)
			 .SupportsMinimize(false)
			 [
			 	SNew(SBorder)
			 	.Padding(4.f)
			 	.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			 	[
			 		SAssignNew(SourceControlFileDialog, SSourceControlFileDialog)
			 		.Message(LOCTEXT("CheckoutPackagesDialogMessage", "Conflict detected in the following assets:"))
			 		.Warning(LOCTEXT("CheckoutPackagesWarnMessage", "Warning: These assets are locked or not at the head revision. You may lose your changes if you continue, as you will be unable to submit them to revision control."))
			 		.Files(InFilesConflicts)
			 	]
			 ];

	SourceControlFileDialog->SetWindow(Window);
	Window->SetWidgetToFocusOnActivate(SourceControlFileDialog);
	GEditor->EditorAddModalWindow(Window.ToSharedRef());

	return SourceControlFileDialog->IsProceedButtonPressed();
}


} // end of namespace SSourceControlCommon

#undef LOCTEXT_NAMESPACE
