// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/VersePath.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "SourceControlAssetDataCache.h"
#include "ISourceControlProvider.h"
#include "UncontrolledChangelistState.h"
#include "Input/DragAndDrop.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Templates/Function.h"

struct FAssetData;

/**
 * Modelizes a changelist node in a source control tree-like structure.
 * The modelized tree stored is as below in memory.
 * 
 * > Changelist
 *     File
 *     > ShelvedChangelist
 *         ShelvedFile
 * 
 * > UncontrolledChangelist
 *     File
 *     Offline File
 *
 * > UnsavedAssets
 *     Offline File
 */
struct IChangelistTreeItem : TSharedFromThis<IChangelistTreeItem>
{
	enum TreeItemType
	{
		Changelist,              // Node displaying a change list description.
		UncontrolledChangelist,  // Node displaying an uncontrolled change list description.
		File,                    // Node displaying a file information.
		ShelvedChangelist,       // Node displaying shelved files as children.
		ShelvedFile,             // Node displaying a shelved file information.
		OfflineFile,             // Node displaying an offline file information.
		UnsavedAssets,           // Node displaying unsaved asset category
	};

	virtual ~IChangelistTreeItem() = default;

	/** Get this item's parent. Can be nullptr for root nodes. */
	TSharedPtr<IChangelistTreeItem> GetParent() const;

	/** Get this item's children, if any. Although we store as weak pointers, they are guaranteed to be valid. */
	const TArray<TSharedPtr<IChangelistTreeItem>>& GetChildren() const;

	/** Returns the TreeItem's type */
	const TreeItemType GetTreeItemType() const { return Type; }

	/** Add a child to this item */
	void AddChild(TSharedRef<IChangelistTreeItem> Child);

	/** Remove a child from this item */
	void RemoveChild(const TSharedRef<IChangelistTreeItem>& Child);

	/** Remove all children from this item. */
	void RemoveAllChildren();

public:
	/** A sequence number representing the last time the item was inspected by the widget owning this UI item. Detect when the underlying model object stopped to exist between two UI updates.*/
	int64 VisitedUpdateNum = -1;

	/** A sequence number representing the last time the item was displayed by the widget owning this item. Used to detect when an existing item started/stopped to be visible between two UI updates.*/
	int64 DisplayedUpdateNum = -1;

protected:
	IChangelistTreeItem(TreeItemType InType) { Type = InType; }

	/** This item's parent, if any. */
	TSharedPtr<IChangelistTreeItem> Parent;

	/** Array of children contained underneath this item */
	TArray<TSharedPtr<IChangelistTreeItem>> Children;

	/** This item type. */
	TreeItemType Type;
};


/**
 * Abstracts the values displayed in the file view that has a set of columns. The API returns values
 * as string rather than FText to avoid conversion when sorting very large collection (as FText convert
 * internally to FString for comparison).
 */
struct IFileViewTreeItem : public IChangelistTreeItem
{
	/** Returns the checkbox state of this item */
	virtual ECheckBoxState GetCheckBoxState() const = 0;

	/** Sets the checkbox state of this item */
	virtual void SetCheckBoxState(ECheckBoxState NewState) = 0;

	/**The 'Priority' given to the item icon when sorting ascending (lower will be sorted first). */
	virtual int32 GetIconSortingPriority() const = 0;

	/** The values displayed in the 'Name' column. */
	virtual const FString& GetName() const = 0;

	/** The values displayed in the 'Path' column when not displaying package names or Verse paths. */
	virtual const FString& GetPath() const = 0;

	/** The values displayed in the 'Path' column when displaying Verse paths. */
	virtual const UE::Core::FVersePath& GetVersePath() const = 0;

	/** The values displayed in the 'Type' column. */
	virtual const FString& GetType() const = 0;

	/** The values displayed in the 'Type Name' column. */
	virtual const FString& GetTypeName() const { return DefaultStrValue; }

	/** The values displayed in the 'User' column. */
	virtual const FString& GetCheckedOutBy() const { return DefaultStrValue; }

	/** Returns the full pathname of the files on the file sytem. */
	virtual const FString& GetFullPathname() const = 0;

	/** Set the last modified time timestamp. */
	void SetLastModifiedDateTime(const FDateTime& Timestamp);

	/** The values displayed in the 'Last Modified' column. */
	const FDateTime& GetLastModifiedDateTime() const { return LastModifiedDateTime; }

	/** The values displayed in the 'Last Modified' column as text. */
	FText GetLastModifiedTimestamp() const { return LastModifiedTimestampText; }

	virtual bool RefreshVersePath() = 0;

public:
	/** Keep the icon sorting priority as it was the last time the item was displayed. Used to detect if the priority changed between two refreshes of the UI. */
	int32 DisplayedIconPriority = -1;

protected:
	IFileViewTreeItem(TreeItemType InType) : IChangelistTreeItem(InType) {}

private:
	// Use an empty string to return as default for const FString&.
	static const FString DefaultStrValue;

	/** The timestamp of the last modification to the file. */
	FText LastModifiedTimestampText;
	FDateTime LastModifiedDateTime;
};


/** Root node to group shelved files as children. */
struct FShelvedChangelistTreeItem : public IChangelistTreeItem
{
	FShelvedChangelistTreeItem() : IChangelistTreeItem(IChangelistTreeItem::ShelvedChangelist) {}
	FText GetDisplayText() const;
};


/** Displays a changelist icon/number/description. */
struct FChangelistTreeItem : public IChangelistTreeItem
{
	FChangelistTreeItem(TSharedRef<ISourceControlChangelistState> InChangelistState)
		: IChangelistTreeItem(IChangelistTreeItem::Changelist)
		, ChangelistState(MoveTemp(InChangelistState))
	{
	}

	FText GetDisplayText() const
	{
		return ChangelistState->GetDisplayText();
	}

	FText GetDescriptionText() const
	{
		return ChangelistState->GetDescriptionText();
	}

	int32 GetFileCount() const
	{
		return ChangelistState->GetFilesStatesNum();
	}

	int32 GetShelvedFileCount() const
	{
		return ChangelistState->GetShelvedFilesStatesNum();
	}

	TSharedRef<ISourceControlChangelistState> ChangelistState;
	TSharedPtr<FShelvedChangelistTreeItem> ShelvedChangelistItem;
};


/** Displays an uncontrolled changelist icon/number/description. */
struct FUncontrolledChangelistTreeItem : public IChangelistTreeItem
{
	FUncontrolledChangelistTreeItem(FUncontrolledChangelistStateRef InUncontrolledChangelistState)
		: IChangelistTreeItem(IChangelistTreeItem::UncontrolledChangelist)
		, UncontrolledChangelistState(InUncontrolledChangelistState)
	{
	}

	FText GetDisplayText() const
	{
		return UncontrolledChangelistState->GetDisplayText();
	}

	FText GetDescriptionText() const
	{
		return UncontrolledChangelistState->GetDescriptionText();
	}

	int32 GetFileCount() const
	{
		return UncontrolledChangelistState->GetFileCount();
	}

	FUncontrolledChangelistStateRef UncontrolledChangelistState;
};

/** Displays a changelist icon/number/description. */
struct FUnsavedAssetsTreeItem : public IChangelistTreeItem
{
	FUnsavedAssetsTreeItem()
		: IChangelistTreeItem(IChangelistTreeItem::UnsavedAssets)
	{
	}

	FString GetDisplayString() const;
};

/** Displays a set of files under a changelist or uncontrolled changelist. */
struct FFileTreeItem : public IFileViewTreeItem
{
	explicit FFileTreeItem(FSourceControlStateRef InFileState, bool bBeautifyPaths = true, bool bIsShelvedFile = false);

	virtual ECheckBoxState GetCheckBoxState() const override { return CheckBoxState; }
	virtual void SetCheckBoxState(ECheckBoxState NewState) override { CheckBoxState = NewState; }
	virtual int32 GetIconSortingPriority() const override;
	virtual const FString& GetName() const override { return AssetNameStr; }
	virtual const FString& GetPath() const override { return AssetPathStr; }
	virtual const UE::Core::FVersePath& GetVersePath() const override { return AssetVersePathStruct; }
	virtual const FString& GetType() const override { return AssetTypeStr; }
	virtual const FString& GetTypeName() const override { return AssetTypeNameStr; }
	virtual const FString& GetFullPathname() const override { return FileState->GetFilename(); }
	virtual const FString& GetCheckedOutBy() const override;

	virtual bool RefreshVersePath() override;

	/** Updates informations based on AssetData */
	void RefreshAssetInformation();

	/** Returns the asset name of the item. This might update the asset names from the asset registry. */
	FText GetAssetName();

	/** Returns the asset name. This returns the currently cached asset name.*/
	FText GetAssetName() const;

	/** Returns the asset path of the item */
	FText GetAssetPath() const { return AssetPath; }

	/** Returns the asset Verse path of the item */
	FText GetAssetVersePath() const { return AssetVersePath; }

	/** Returns the asset type of the item */
	FText GetAssetType() const { return AssetType; }

	/** Returns the asset type name of the item */
	FText GetAssetTypeName() const { return AssetTypeName; }

	/** Returns the asset type color of the item */
	FSlateColor GetAssetTypeColor() const { return FSlateColor(AssetTypeColor); }

	/** Returns the user that checked out the file/asset (if any). */
	FText GetCheckedOutByUser() const;

	/** Returns the file name of the item in source control */
	FText GetFileName() const;

	/** true if the item is not in source control and needs to be added prior to checkin */
	bool NeedsAdding() const { return !FileState->IsSourceControlled(); }

	/** true if the item is in source control and is able to be checked in */
	bool CanCheckIn() const { return FileState->CanCheckIn() || FileState->IsDeleted(); }

	/** true if the item is enabled in the list */
	bool IsEnabled() const { return !FileState->IsConflicted() && FileState->IsCurrent(); }

	/** true if the item is source controlled and not marked for add nor for delete */
	bool CanDiff() const { return FileState->IsSourceControlled() && !FileState->IsAdded() && !FileState->IsDeleted(); }

	/** true if the item is source controlled and can be reverted */
	bool CanRevert() const { return FileState->IsSourceControlled() && FileState->CanRevert(); }

	const FAssetDataArrayPtr& GetAssetData() const
	{
		return Assets;
	}

	bool IsShelved() const { return GetTreeItemType() == IChangelistTreeItem::ShelvedFile; }

public:
	/** Shared pointer to the source control state object itself */
	FSourceControlStateRef FileState;

private:
	/** Checkbox state */
	ECheckBoxState CheckBoxState;

	/** Cached asset name to display */
	FText AssetName;
	FString AssetNameStr;

	/** Cached asset path to display */
	FText AssetPath;
	FString AssetPathStr;

	/** Cached Verse path to display */
	FText AssetVersePath;
	UE::Core::FVersePath AssetVersePathStruct;

	/** Cached asset type to display */
	FText AssetType;
	FString AssetTypeStr;

	/** Cached asset type name to display */
	FText AssetTypeName;
	FString AssetTypeNameStr;

	/** Cached asset type related color to display */
	FColor AssetTypeColor;

	/** The other user that has the checked out. */
	mutable FString CheckedOutBy;

	/** Matching asset(s) to facilitate Locate in content browser */
	FAssetDataArrayPtr Assets;

	/** Represents the minimum amount of time between to attempt to refresh AssetData */
	const FTimespan MinTimeBetweenUpdate;

	/** Timestamp representing the time at which the last information update was made */
	FTimespan LastUpdateTime;

	/** True if informations returned from the cache are up to date */
	bool bAssetsUpToDate;
};


struct FShelvedFileTreeItem : public FFileTreeItem
{
	explicit FShelvedFileTreeItem(FSourceControlStateRef InFileState, bool bBeautifyPaths = true)
		: FFileTreeItem(InFileState, bBeautifyPaths,/*bIsShelved=*/true)
	{
	}
};


struct FOfflineFileTreeItem : public IFileViewTreeItem
{
	explicit FOfflineFileTreeItem(const FString& InFilename);

	void RefreshAssetInformation();

public:
	virtual ECheckBoxState GetCheckBoxState() const override { return CheckBoxState; }
	virtual void SetCheckBoxState(ECheckBoxState NewState) override { CheckBoxState = NewState; }
	virtual int32 GetIconSortingPriority() const override { return 0; }
	virtual const FString& GetName() const override { return AssetNameStr; }
	virtual const FString& GetPath() const override { return AssetPathStr; }
	virtual const UE::Core::FVersePath& GetVersePath() const override { return AssetVersePathStruct; }
	virtual const FString& GetType() const override { return AssetTypeStr; }
	virtual const FString& GetFullPathname() const override { return FilenameStr; }

	virtual bool RefreshVersePath() override;

	const FText& GetDisplayFilename() const { return Filename; }
	const FText& GetDisplayName() const { return AssetName; }
	const FText& GetDisplayPath() const { return AssetPath; }
	const FText& GetDisplayVersePath() const { return AssetVersePath; }
	const FText& GetDisplayType() const { return AssetType; }
	const FColor& GetDisplayColor() const { return AssetTypeColor; }

private:
	ECheckBoxState CheckBoxState;
	TArray<FAssetData> Assets;
	FText Filename;
	FString FilenameStr;
	FText AssetName;
	FString AssetNameStr;
	FText AssetPath;
	FString AssetPathStr;
	FText AssetVersePath;
	UE::Core::FVersePath AssetVersePathStruct;
	FText AssetType;
	FString AssetTypeStr;
	FText AssetTypeName;
	FString AssetTypeNameStr;
	FColor AssetTypeColor;
};


namespace SSourceControlCommon
{
	TSharedRef<SWidget> GetSCCStatusWidget(FSourceControlStateRef InFileState);
	TSharedRef<SWidget> GetSCCStatusWidget();
	TSharedRef<SWidget> GetSCCShelveWidget(bool bIsShelvedFile);
	TSharedRef<SWidget> GetSCCShelveWidget();
	FText GetDefaultAssetName();
	FText GetDefaultAssetType();
	FText GetDefaultUnknownAssetType();
	FText GetDefaultMultipleAsset();

	enum ESingleLineFlags
	{
		NewlineTerminates		=0x0,
		NewlineConvertToSpace	=0x1,
		Mask_NewlineBehavior	=0x1,
	};
	ENUM_CLASS_FLAGS(ESingleLineFlags);
	/**
	 * returns the first non-whitespace line, or an empty FText if InFullDescription is empty or only whitespace
	 */
	FText GetSingleLineChangelistDescription(const FText& InFullDescription, ESingleLineFlags Flags = ESingleLineFlags::NewlineTerminates);

	void ExecuteChangelistOperationWithSlowTaskWrapper(const FText& Message, const TFunction<void()>& ChangelistTask);
	void ExecuteUncontrolledChangelistOperationWithSlowTaskWrapper(const FText& Message, const TFunction<void()>& UncontrolledChangelistTask);
	
	void DisplaySourceControlOperationNotification(const FText& Message, SNotificationItem::ECompletionState CompletionState);
	void DisplaySourceControlOperationNotification(const FNotificationInfo& NotificationInfo, SNotificationItem::ECompletionState CompletionState);
	TOptional<FNotificationInfo> ConstructSourceControlOperationNotification(const FText& Message);

	bool OpenConflictDialog(const TArray<FSourceControlStateRef>& InFilesConflicts);
}


/** Implements drag and drop operation. */
struct FSCCFileDragDropOp : public FDragDropOperation
{
	DRAG_DROP_OPERATOR_TYPE(FSCCFileDragDropOp, FDragDropOperation);

	using FDragDropOperation::Construct;

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		// Offline files won't coexist with Files
		if (!OfflineFiles.IsEmpty())
		{
			return SSourceControlCommon::GetSCCStatusWidget();
		}

		FSourceControlStateRef FileState = Files.IsEmpty() ? UncontrolledFiles[0] : Files[0];
		return SSourceControlCommon::GetSCCStatusWidget(MoveTemp(FileState));
	}

	TArray<FSourceControlStateRef> Files;
	TArray<FSourceControlStateRef> UncontrolledFiles;
	TArray<FString> OfflineFiles;
};


typedef TSharedPtr<FUncontrolledChangelistTreeItem> FUncontrolledChangelistTreeItemPtr;
typedef TSharedRef<FUncontrolledChangelistTreeItem> FUncontrolledChangelistTreeItemRef;
typedef TSharedPtr<IChangelistTreeItem> FChangelistTreeItemPtr;
typedef TSharedRef<IChangelistTreeItem> FChangelistTreeItemRef;
typedef TSharedPtr<FFileTreeItem> FFileTreeItemPtr;
typedef TSharedRef<FFileTreeItem> FFileTreeItemRef;
