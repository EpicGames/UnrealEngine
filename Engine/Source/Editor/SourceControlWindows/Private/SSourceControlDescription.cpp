// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSourceControlDescription.h"

#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Input/Events.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/Vector2D.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "Types/SlateEnums.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
struct FGeometry;

#define LOCTEXT_NAMESPACE "SourceControl.Description"

void SSourceControlDescriptionWidget::Construct(const FArguments& InArgs)
{
	ParentWindow = InArgs._ParentWindow.Get();
	Items = InArgs._Items;

	const bool bHasItems = (Items && Items->Num() > 0);
	const bool bShowSelectionDropDown = bHasItems;
	const bool bDescriptionCanBeEdited = (!bHasItems || (*Items)[0].bCanEditDescription);

	CurrentlySelectedItemIndex = 0;

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(16)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(InArgs._Label)
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(8, 0, 0, 0)
				[
					SNew(SComboButton)
					.Visibility(bShowSelectionDropDown ? EVisibility::Visible : EVisibility::Hidden)
					.OnGetMenuContent(this, &SSourceControlDescriptionWidget::GetSelectionContent)
					.ButtonContent()
					[
						SNew(STextBlock)
						.Text(this, &SSourceControlDescriptionWidget::GetSelectedItemTitle)
					]
				]
			]
			+ SVerticalBox::Slot()
			.Padding(FMargin(16, 0, 16, 16))
			[
				SAssignNew(TextBox, SMultiLineEditableTextBox)
				.SelectAllTextWhenFocused(bDescriptionCanBeEdited)
				.IsReadOnly(!bDescriptionCanBeEdited)
				.AutoWrapText(true)
				.Text(InArgs._Text)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(16)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				+SUniformGridPanel::Slot(0,0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.Text( NSLOCTEXT("SourceControl.Description", "OKButton", "Ok") )
					.OnClicked(this, &SSourceControlDescriptionWidget::OKClicked)
				]
				+SUniformGridPanel::Slot(1,0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.Text( NSLOCTEXT("SourceControl.Description", "CancelButton", "Cancel") )
					.OnClicked(this, &SSourceControlDescriptionWidget::CancelClicked)
				]
			]
		]
	];

	ParentWindow.Pin()->SetWidgetToFocusOnActivate(TextBox);
}

FReply SSourceControlDescriptionWidget::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	// Pressing escape returns as if the user clicked cancel
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		return CancelClicked();
	}

	return FReply::Unhandled();
}

FReply SSourceControlDescriptionWidget::OKClicked()
{
	bResult = true;
	ParentWindow.Pin()->RequestDestroyWindow();

	return FReply::Handled();
}


FReply SSourceControlDescriptionWidget::CancelClicked()
{
	bResult = false;
	ParentWindow.Pin()->RequestDestroyWindow();

	return FReply::Handled();
}

FText SSourceControlDescriptionWidget::GetDescription() const
{
	return TextBox->GetText();
}

static FText GetTitleAndShortDescription(const FText& InTitle, const FText& InDescription)
{
	if (InDescription.IsEmptyOrWhitespace())
	{
		return InTitle;
	}
	else
	{
		FString TrimmedDescription = InDescription.ToString();
		
		const int32 TrimmedMaxLen = 50;
		if (TrimmedDescription.Len() > TrimmedMaxLen)
		{
			TrimmedDescription = TrimmedDescription.Left(TrimmedMaxLen);
			TrimmedDescription += TEXT("...");
		}
		
		TrimmedDescription.ReplaceInline(TEXT("\r"), TEXT(""));
		TrimmedDescription.ReplaceInline(TEXT("\n"), TEXT(" "));

		FString TitleAndShortDescription = InTitle.ToString();
		TitleAndShortDescription += TEXT(": ");
		TitleAndShortDescription += TrimmedDescription;

		return FText::FromString(TitleAndShortDescription);
	}
}

FText SSourceControlDescriptionWidget::GetSelectedItemTitle() const
{
	if (CurrentlySelectedItemIndex >= 0 && Items && CurrentlySelectedItemIndex < Items->Num())
	{
		const SSourceControlDescriptionItem& Item = (*Items)[CurrentlySelectedItemIndex];
		return GetTitleAndShortDescription(Item.Title, Item.Description);
	}
	else
	{
		return LOCTEXT("SourceControlDescription_InvalidItemTitle", "Invalid");
	}
}

TSharedRef<SWidget> SSourceControlDescriptionWidget::GetSelectionContent()
{
	if (!Items || Items->Num() == 0)
	{
		return SNullWidget::NullWidget;
	}

	FMenuBuilder MenuBuilder(true, nullptr);

	for(int32 ItemIndex = 0; ItemIndex < Items->Num(); ++ItemIndex)
	{
		const SSourceControlDescriptionItem& Item = (*Items)[ItemIndex];

		MenuBuilder.AddMenuEntry(
			GetTitleAndShortDescription(Item.Title, Item.Description),
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, ItemIndex, &Item]() {
					TextBox->SetIsReadOnly(!Item.bCanEditDescription);
					TextBox->SetText(Item.Description);
					if (Item.bCanEditDescription)
					{
						TextBox->SelectAllText();
					}
					
					CurrentlySelectedItemIndex = ItemIndex;
					})),
			NAME_None
		);
	}

	return MenuBuilder.MakeWidget();
}

bool GetChangelistDescription(
	const TSharedPtr<SWidget>& ParentWidget,
	const FText& InWindowTitle, 
	const FText& InLabel, 
	FText& OutDescription)
{
	FText InitialDescription = OutDescription;
	if (InitialDescription.IsEmpty())
	{
		InitialDescription = LOCTEXT("SourceControl.NewDescription", "<enter description here>");
	}

	TSharedRef<SWindow> NewWindow = SNew(SWindow)
		.Title(InWindowTitle)
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(600, 400))
		.SupportsMaximize(true)
		.SupportsMinimize(false);

	TSharedRef<SSourceControlDescriptionWidget> SourceControlWidget =
		SNew(SSourceControlDescriptionWidget)
		.ParentWindow(NewWindow)
		.Label(InLabel)
		.Text(InitialDescription);

	NewWindow->SetContent(SourceControlWidget);

	FSlateApplication::Get().AddModalWindow(NewWindow, ParentWidget);

	if (SourceControlWidget->GetResult())
	{
		OutDescription = SourceControlWidget->GetDescription();
	}

	return SourceControlWidget->GetResult();
}

bool PickChangelistOrNewWithDescription(
	const TSharedPtr<SWidget>& ParentWidget,
	const FText& InWindowTitle,
	const FText& InLabel,
	const TArray<SSourceControlDescriptionItem>& Items,
	int32& OutPickedIndex,
	FText& OutDescription)
{
	if (Items.Num() == 0)
	{
		return false;
	}

	TSharedRef<SWindow> NewWindow = SNew(SWindow)
		.Title(InWindowTitle)
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(600, 400))
		.SupportsMaximize(true)
		.SupportsMinimize(false);

	TSharedRef<SSourceControlDescriptionWidget> SourceControlWidget =
		SNew(SSourceControlDescriptionWidget)
		.ParentWindow(NewWindow)
		.Label(InLabel)
		.Text(Items[0].Description)
		.Items(&Items);

	NewWindow->SetContent(SourceControlWidget);

	FSlateApplication::Get().AddModalWindow(NewWindow, ParentWidget);

	if (SourceControlWidget->GetResult())
	{
		OutPickedIndex = SourceControlWidget->GetSelectedItemIndex();
		OutDescription = SourceControlWidget->GetDescription();
	}

	return SourceControlWidget->GetResult();
}

#undef LOCTEXT_NAMESPACE