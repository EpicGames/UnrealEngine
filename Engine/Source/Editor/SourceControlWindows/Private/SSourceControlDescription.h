// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SMultiLineEditableTextBox;
class SWidget;
class SWindow;
struct FGeometry;
struct FKeyEvent;

struct SSourceControlDescriptionItem
{
	SSourceControlDescriptionItem(const FText& InTitle, const FText& InDescription, bool bInCanEditDescription)
		: Title(InTitle), Description(InDescription), bCanEditDescription(bInCanEditDescription)
	{}

	FText Title;
	FText Description;
	bool bCanEditDescription;
};

class SSourceControlDescriptionWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSourceControlDescriptionWidget)
		: _ParentWindow()
		, _Label()
		, _Text()
		, _Items()
	{}

		SLATE_ATTRIBUTE(TSharedPtr<SWindow>, ParentWindow)
		SLATE_ATTRIBUTE(FText, Label)
		SLATE_ATTRIBUTE(FText, Text)
		SLATE_ARGUMENT(const TArray<SSourceControlDescriptionItem>*, Items)

	SLATE_END_ARGS()

	/** Constructs the widget */
	void Construct(const FArguments& InArgs);

	/** Gets dialog result */
	bool GetResult() const { return bResult; }

	/** Used to intercept Escape key press, and interpret it as cancel */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	/** Returns the text currently in the edit box */
	FText GetDescription() const;

	/** Returns the currently selected item index if any */
	int32 GetSelectedItemIndex() const { return CurrentlySelectedItemIndex; }

private:
	/** Called when the settings of the dialog are to be accepted*/
	FReply OKClicked();

	/** Called when the settings of the dialog are to be ignored*/
	FReply CancelClicked();

	/** Called to populate the dropdown */
	TSharedRef<SWidget> GetSelectionContent();

	/** Returns title of currently selected item */
	FText GetSelectedItemTitle() const;

private:
	bool bResult = false;

	/** Pointer to the parent modal window */
	TWeakPtr<SWindow> ParentWindow;

	TSharedPtr< SMultiLineEditableTextBox> TextBox;

	const TArray<SSourceControlDescriptionItem>* Items;
	int32 CurrentlySelectedItemIndex;
};

bool GetChangelistDescription(
	const TSharedPtr<SWidget>& ParentWidget,
	const FText& InWindowTitle,
	const FText& InLabel,
	FText& OutDescription);

bool PickChangelistOrNewWithDescription(
	const TSharedPtr<SWidget>& ParentWidget,
	const FText& InWindowTitle,
	const FText& InLabel,
	const TArray<SSourceControlDescriptionItem>& Items,
	int32& OutPickedIndex,
	FText& OutDescription);