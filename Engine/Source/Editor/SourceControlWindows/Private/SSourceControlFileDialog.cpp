// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSourceControlFileDialog.h"

#include "AssetToolsModule.h"
#include "Framework/Application/SlateApplication.h"
#include "IAssetTools.h"
#include "RevisionControlStyle/RevisionControlStyle.h"
#include "SSourceControlChangelistRows.h"
#include "Styling/AppStyle.h"
#include "SWarningOrErrorBox.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBorder.h"

#define LOCTEXT_NAMESPACE "SSourceControlFileDialog"

void SSourceControlFileDialog::Construct(const FArguments& InArgs)
{
	SortByColumn = SourceControlFileViewColumn::Name::Id();

	bShowingContentVersePath = FAssetToolsModule::GetModule().Get().ShowingContentVersePath();

	Reset();
	SetMessage(InArgs._Message);
	SetWarning(InArgs._Warning);
	SetFiles(InArgs._Files);

	FileTreeView =	SNew(STreeView<FChangelistTreeItemPtr>)
					.TreeItemsSource(&FileTreeNodes)
					.OnGenerateRow(this, &SSourceControlFileDialog::OnGenerateRow)
					.OnGetChildren(this, &SSourceControlFileDialog::OnGetFileChildren)
					.HeaderRow
					(
						SNew(SHeaderRow)
						+ SHeaderRow::Column(SourceControlFileViewColumn::Icon::Id())
						.DefaultTooltip(SourceControlFileViewColumn::Icon::GetToolTipText())
						.FillSized(18)
						.HeaderContentPadding(FMargin(0.0f))
						.SortMode(this, &SSourceControlFileDialog::GetColumnSortMode, SourceControlFileViewColumn::Icon::Id())
						.OnSort(this, &SSourceControlFileDialog::OnColumnSortModeChanged)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.Padding(1.0f, 0.0f)
							[
								SNew(SBox)
								.WidthOverride(16.0f)
								.HeightOverride(16.0f)
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								.Visibility(this, &SSourceControlFileDialog::GetIconColumnContentVisibility)
								[
									SNew(SImage)
									.ColorAndOpacity(FSlateColor::UseSubduedForeground())
									.Image(FRevisionControlStyleManager::Get().GetBrush("RevisionControl.ChangelistsTab"))
								]
							]
						]

						+ SHeaderRow::Column(SourceControlFileViewColumn::Name::Id())
						.DefaultLabel(SourceControlFileViewColumn::Name::GetDisplayText())
						.DefaultTooltip(SourceControlFileViewColumn::Name::GetToolTipText())
						.FillWidth(0.2f)
						.SortMode(this, &SSourceControlFileDialog::GetColumnSortMode, SourceControlFileViewColumn::Name::Id())
						.OnSort(this, &SSourceControlFileDialog::OnColumnSortModeChanged)

						+ SHeaderRow::Column(SourceControlFileViewColumn::Path::Id())
						.DefaultLabel(SourceControlFileViewColumn::Path::GetDisplayText())
						.DefaultTooltip(SourceControlFileViewColumn::Path::GetToolTipText())
						.FillWidth(0.6f)
						.SortMode(this, &SSourceControlFileDialog::GetColumnSortMode, SourceControlFileViewColumn::Path::Id())
						.OnSort(this, &SSourceControlFileDialog::OnColumnSortModeChanged)

						+ SHeaderRow::Column(SourceControlFileViewColumn::Type::Id())
						.DefaultLabel(SourceControlFileViewColumn::Type::GetDisplayText())
						.DefaultTooltip(SourceControlFileViewColumn::Type::GetToolTipText())
						.FillWidth(0.2f)
						.SortMode(this, &SSourceControlFileDialog::GetColumnSortMode, SourceControlFileViewColumn::Type::Id())
						.OnSort(this, &SSourceControlFileDialog::OnColumnSortModeChanged)
					);

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(FMargin(16))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(0.1)
			.Padding(0.0f, 0.0f, 0.0f, 8.0f)
			[
				SNew(STextBlock)
				.Text(this, &SSourceControlFileDialog::GetMessage)
				.AutoWrapText(true)
			]
			+ SVerticalBox::Slot()
			.FillHeight(0.8)
			[
				SNew(SScrollBorder, FileTreeView.ToSharedRef())
				[
					FileTreeView.ToSharedRef()
				]
			]
			+ SVerticalBox::Slot()
			.Padding(0, 16.0f, 0, 0)
			.AutoHeight()
			[
				SNew(SWarningOrErrorBox)
				.Visibility(this, &SSourceControlFileDialog::GetWarningVisibility)
				.Message(this, &SSourceControlFileDialog::GetWarning)
			]
			+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 16.0f, 0.0f, 0.0f)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Bottom)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(5, 0)
				[
					SAssignNew(ProceedButton, SButton)
					.ButtonStyle(&FAppStyle::Get(), "PrimaryButton")
					.TextStyle(&FAppStyle::Get(), "PrimaryButtonText")
					.Text(LOCTEXT("Proceed", "Proceed"))
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.OnClicked(this, &SSourceControlFileDialog::OnProceedClicked)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(5, 0)
				[
					SAssignNew(CancelButton, SButton)
					.ButtonStyle(&FAppStyle::Get(), "Button")
					.TextStyle(&FAppStyle::Get(), "ButtonText")
					.Text(LOCTEXT("Cancel", "Cancel"))
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.OnClicked(this, &SSourceControlFileDialog::OnCancelClicked)
				]
			]
		]
	];
}

void SSourceControlFileDialog::SetMessage(const FText& InMessage)
{
	Message = InMessage;
}

void SSourceControlFileDialog::SetWarning(const FText& InWarning)
{
	Warning = InWarning;
}

void SSourceControlFileDialog::SetFiles(const TArray<FSourceControlStateRef>& InFiles)
{
	for (const TSharedRef<ISourceControlState>& FileState : InFiles)
	{
		FileTreeNodes.Add(MakeShared<FFileTreeItem>(FileState, true));
	}

	SortFiles();

	if (FileTreeView.IsValid())
	{
		FileTreeView->RequestTreeRefresh();
	}
}

EColumnSortMode::Type SSourceControlFileDialog::GetColumnSortMode(FName ColumnId) const
{
	if (SortByColumn != ColumnId)
	{
		return EColumnSortMode::None;
	}

	return SortMode;
}

void SSourceControlFileDialog::OnColumnSortModeChanged(EColumnSortPriority::Type SortPriority, const FName& ColumnId, EColumnSortMode::Type InSortMode)
{
	SortByColumn = ColumnId;
	SortMode = InSortMode;

	SortFiles();
	FileTreeView->RequestListRefresh();
}

EVisibility SSourceControlFileDialog::GetIconColumnContentVisibility() const
{
	// Hide the icon when sorting the icon column (it clashes with the sort mode icon).
	return GetColumnSortMode(SourceControlFileViewColumn::Icon::Id()) == EColumnSortMode::None ? EVisibility::Visible : EVisibility::Collapsed;
}

void SSourceControlFileDialog::SortFiles()
{
	TFunction<bool(const IFileViewTreeItem&, const IFileViewTreeItem&)> SortPredicate = SourceControlFileViewColumn::GetSortPredicate(
		SortMode, SortByColumn, bShowingContentVersePath ? SourceControlFileViewColumn::EPathFlags::ShowingVersePath : SourceControlFileViewColumn::EPathFlags::Default);
	if (SortPredicate)
	{
		Algo::SortBy(
			FileTreeNodes,
			[](const FChangelistTreeItemPtr& ListViewItem) -> const IFileViewTreeItem&
			{
				return static_cast<IFileViewTreeItem&>(*ListViewItem);
			},
			SortPredicate);
	}
}

TSharedRef<ITableRow> SSourceControlFileDialog::OnGenerateRow(TSharedPtr<IChangelistTreeItem> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return	SNew(SFileTableRow, OwnerTable)
			.TreeItemToVisualize(InItem)
			.PathFlags(bShowingContentVersePath ? SourceControlFileViewColumn::EPathFlags::ShowingVersePath : SourceControlFileViewColumn::EPathFlags::Default);
}

void SSourceControlFileDialog::Reset()
{
	bIsProceedButtonPressed = false;
}

void SSourceControlFileDialog::SetWindow(TSharedPtr<SWindow> InWindow)
{
	Window = MoveTemp(InWindow);
}

FReply SSourceControlFileDialog::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if ((InKeyEvent.GetKey() == EKeys::Enter && ProceedButton.IsValid()))
	{
		return OnProceedClicked();
	}
	else if ((InKeyEvent.GetKey() == EKeys::Escape && CancelButton.IsValid()))
	{
		return OnCancelClicked();
	}

	return FReply::Unhandled();
}

FText SSourceControlFileDialog::GetMessage() const
{
	return Message;
}

FText SSourceControlFileDialog::GetWarning() const
{
	return Warning;
}

EVisibility SSourceControlFileDialog::GetWarningVisibility() const
{
	return (Warning.IsEmpty()) ? EVisibility::Collapsed : EVisibility::HitTestInvisible;
}

FReply SSourceControlFileDialog::OnProceedClicked()
{
	bIsProceedButtonPressed = true;
	CloseDialog();

	return FReply::Handled();
}

FReply SSourceControlFileDialog::OnCancelClicked()
{
	bIsProceedButtonPressed = false;
	CloseDialog();

	return FReply::Handled();
}

void SSourceControlFileDialog::CloseDialog()
{
	if (Window.IsValid())
	{
		Window->RequestDestroyWindow();
	}
}

#undef LOCTEXT_NAMESPACE
