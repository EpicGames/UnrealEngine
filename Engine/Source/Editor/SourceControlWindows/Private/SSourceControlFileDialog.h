// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "SSourceControlCommon.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

/**
 * Represents dialog displaying a provided message along with source control files
 */
class SSourceControlFileDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSourceControlFileDialog)
		: _Message()
		, _Warning()
		, _Files()
	{}

	/** The message of the widget */
	SLATE_ARGUMENT(FText, Message)

	/** The warning message of the widget */
	SLATE_ARGUMENT(FText, Warning)

	/** The Source Control Files to display */
	SLATE_ARGUMENT(TArray<FSourceControlStateRef>, Files)

	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 *
	 * @param	InArgs	The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs);

	/**
	 * Sets the message of the widget
	 *
	 * @param	InMessage	The string that the message should be set to
	 */
	void SetMessage(const FText& InMessage);

	/**
	 * Sets the warning message of the widget
	 *
	 * @param	InMessage	The string that the warning message should be set to
	 */
	void SetWarning(const FText& InMessage);

	/**
	 * Sets source control files to display
	 *
	 * @param	InFiles		The Source Control Files to display
	 */
	void SetFiles(const TArray<FSourceControlStateRef>& InFiles);

	/** Reset the state of this dialogs buttons */
	void Reset();

	/** Returns true if the user pressed the primary button */
	bool IsProceedButtonPressed() const { return bIsProceedButtonPressed; }

	/**
	 * Sets the owner's window handle
	 * @param	InWindow	The window owning the dialog
	 */
	void SetWindow(TSharedPtr<SWindow> InWindow);

private:
	//~ Begin SWidget Interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget Interface

	EColumnSortMode::Type GetColumnSortMode(FName ColumnId) const;

	void OnColumnSortModeChanged(EColumnSortPriority::Type SortPriority, const FName& ColumnId, EColumnSortMode::Type InSortMode);

	EVisibility GetIconColumnContentVisibility() const;

	void SortFiles();

	/** Treeview callback */
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<IChangelistTreeItem> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Treeview callback */
	void OnGetFileChildren(FChangelistTreeItemPtr InParent, TArray<FChangelistTreeItemPtr>& OutChildren) {}

	/** Used to supply message text to the widget */
	FText GetMessage() const;

	/** Used to supply warning text to the widget */
	FText GetWarning() const;

	/** Used to determine visibility of the warning */
	EVisibility GetWarningVisibility() const;

	/** Called when the proceed button is clicked */
	FReply OnProceedClicked();

	/** Called when the cancel button is clicked */
	FReply OnCancelClicked();

	/** Closes the dialog by requesting the owning window to be destroyed */
	void CloseDialog();

private:
	/** The Window owning the dialog */
	TSharedPtr<SWindow> Window;

	FName SortByColumn;
	EColumnSortMode::Type SortMode = EColumnSortMode::Ascending;

	/** Hold the nodes displayed by the file tree. */
	TArray<TSharedPtr<IChangelistTreeItem>> FileTreeNodes;

	/** Display the list of files associated to the selected changelist, uncontrolled changelist or shelved node. */
	TSharedPtr<STreeView<FChangelistTreeItemPtr>> FileTreeView;

	/** The primary button widget. */
	TSharedPtr<SButton> ProceedButton;

	/** The secondary button widget. */
	TSharedPtr<SButton> CancelButton;

	/** The message to display */
	FText Message;

	/** The warning to display */
	FText Warning;

	bool bShowingContentVersePath = false;

	/** Set to true if the user clicked on the primary button */
	bool bIsProceedButtonPressed = false;
};
