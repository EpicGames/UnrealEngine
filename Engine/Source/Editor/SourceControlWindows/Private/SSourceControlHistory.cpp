// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "ISourceControlRevision.h"
#include "SourceControlWindows.h"
#include "SourceControlHelpers.h"
#include "ISourceControlModule.h"
#include "SSourceControlCommon.h"
#include "Modules/ModuleManager.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "InputCoreTypes.h"
#include "Layout/Visibility.h"
#include "Layout/Geometry.h"
#include "Widgets/SNullWidget.h"
#include "Styling/SlateBrush.h"
#include "Input/Events.h"
#include "Input/DragAndDrop.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "SlateOptMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Views/SExpanderArrow.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "Framework/Docking/TabManager.h"
#include "Styling/AppStyle.h"

#include "IAssetTools.h"
#include "IAssetTypeActions.h"
#include "AssetToolsModule.h"
#include "DiffUtils.h"
#include "ToolMenu.h"
#include "ToolMenus.h"

/**
 * Wrapper around data from ISourceControlRevision
 */
class FHistoryRevisionListViewItem 
{
public:
	/** Changelist description */
	FString Description;

	/** User name of submitter */
	FString UserName;

	/** Clientspec/workspace of submitter */
	FString ClientSpec;

	/** File action for this revision (branch, delete, edit, etc.) */
	FString Action;

	/** Source path of branch, if any */
	FString BranchSource;

	/** Date of this revision */
	FDateTime Date;

	/** Number of this revision */
	FString Revision;

	/** Changelist number */
	int32 ChangelistNumber;

	/** Filesize for this revision (0 in the event of a deletion) */
	int32 FileSize;

	/**
	 * Constructor
	 *
	 * @param	InRevision	File revision info. to populate this wrapper with
	 */
	FHistoryRevisionListViewItem( const TSharedRef<ISourceControlRevision, ESPMode::ThreadSafe>& InRevision )
	{
		Description = InRevision->GetDescription();
		UserName = InRevision->GetUserName();
		ClientSpec = InRevision->GetClientSpec();
		Action = InRevision->GetAction();
		BranchSource = InRevision->GetBranchSource().IsValid() ? InRevision->GetBranchSource()->GetFilename() : TEXT("");
		Date = InRevision->GetDate();
		Revision = InRevision->GetRevision();
		ChangelistNumber = InRevision->GetCheckInIdentifier();
		FileSize = InRevision->GetFileSize();
	}
};



/** 
 * Managed mirror of FSourceControlFileHistoryInfo. Designed to represent the history of a file in
 * a listview.
 */
class FHistoryFileListViewItem
{
public:

	/** Depot name of the file */
	FString FileName;

	/**
	 * Constructor
	 *
	 * @param	InFileName		File name of the list item
	 */
	FHistoryFileListViewItem( const FString& InFileName )
		: FileName(InFileName)
	{
	}
};


/**
 * A container class to use the tree view to represent a dynamically expandable nested list
 */
struct FHistoryTreeItem
{
	// Only one of FileListItem or RevisionListItem should be set

	/** Pointer to file info */
	TSharedPtr<FHistoryFileListViewItem> FileListItem;
	/** Pointer to revision info */
	TSharedPtr<FHistoryRevisionListViewItem> RevisionListItem;

	/** If we are a revision entry, pointer to file entry that owns us */
	TWeakPtr<FHistoryTreeItem> Parent;
	/** List of revisions if we are  file entry */
	TArray< TSharedPtr<FHistoryTreeItem> > Children;
};

/**
 * Attempts to get a file list-item that represents the file that specified 
 * history-tree entry belongs to.
 * 
 * @param  HistoryTreeItemIn	The history-tree entry that you want a file item for.
 * @return The file list-item that the specified entry conceptually belongs to (invalid if HistoryTreeItemIn was invalid).
 */
static TSharedPtr<FHistoryFileListViewItem> GetFileListItem(TSharedPtr<FHistoryTreeItem> HistoryTreeItemIn)
{
	TSharedPtr<FHistoryFileListViewItem> FileListItem;

	if (HistoryTreeItemIn.IsValid())
	{
		FileListItem = HistoryTreeItemIn->FileListItem;

		// if this isn't a file list-item itself
		if (!FileListItem.IsValid())
		{
			// then it should have a parent that is one
			TSharedPtr<FHistoryTreeItem> ParentFileItem = HistoryTreeItemIn->Parent.Pin();
			check(ParentFileItem.IsValid());
			check(ParentFileItem->FileListItem.IsValid());
			FileListItem = ParentFileItem->FileListItem;
		}
	}

	return FileListItem;
}

/**
 * Takes a history-tree entry and attempts to find a corresponding asset object 
 * for the specified revision. If the specified history item doesn't have a valid 
 * RevisionListItem (it's a file list-item), we take that to represent the current 
 * working version of the asset.
 * 
 * @param  HistoryTreeItemIn	The history-tree entry that you want a corresponding object for.
 * @return A UObject that represents the asset at the specified revision (NULL if we failed to find/create one).
 */
static UObject* GetAssetRevisionObject(TSharedPtr<FHistoryTreeItem> HistoryTreeItemIn)
{
	UObject* AssetObject = NULL;
	if (HistoryTreeItemIn.IsValid())
	{
		UPackage* AssetPackage = NULL; // need a package to find the asset in

		TSharedPtr<FHistoryFileListViewItem> FileListItem = GetFileListItem(HistoryTreeItemIn);
		check(FileListItem.IsValid());

		TSharedPtr<FHistoryRevisionListViewItem> RevisionListItem = HistoryTreeItemIn->RevisionListItem;
		// if this item is referencing a specific revision (and not the current working version of the asset)
		if (RevisionListItem.IsValid()) // else, 
		{
			// grab details on this file's state in source control (history, etc.)
			ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
			FSourceControlStatePtr FileSourceControlState = SourceControlProvider.GetState(FileListItem->FileName, EStateCacheUsage::Use);

			if (FileSourceControlState.IsValid())
			{
				// lookup the specific revision we want
				TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> FileRevision = FileSourceControlState->FindHistoryRevision(RevisionListItem->Revision);

				FString TempPackageName;
				if (FileRevision.IsValid())
				{
					// try and load the temporary package
					AssetPackage = DiffUtils::LoadPackageForDiff(FileRevision);
				}
			} // if FileSourceControlState.IsValid()
		}
		else // if we want the current working version of this asset
		{
			FString AssetPackageName;
			if (FPackageName::TryConvertFilenameToLongPackageName(FileListItem->FileName, /*out*/ AssetPackageName))
			{
				AssetPackage = FindObject<UPackage>(NULL, *AssetPackageName);
			}
		}

		// grab the asset from the package - we assume asset name matches file name
		FString AssetName = FPaths::GetBaseFilename(FileListItem->FileName);
		AssetObject = FindObject<UObject>(AssetPackage, *AssetName);
		if (AssetPackage && !AssetObject)
		{
			AssetObject = AssetPackage->FindAssetInPackage();
		}

	} // if HistoryTreeItemIn.IsValid()
	
	return AssetObject; 
}


/**
 * Constructs revision info for the specified history-tree entry.
 * 
 * @param  HistoryTreeItemIn	The history-tree entry that you want revision info for.
 * @param  RevisionInfoOut		The resulting revision info (out). 
 */
static void GetRevisionInfo(TSharedPtr<FHistoryTreeItem> HistoryTreeItemIn, FRevisionInfo& RevisionInfoOut)
{
	RevisionInfoOut.Revision = TEXT(""); // clear the revision info (empty string is used signify the current working version)

	// if this is a specific revision item
	if (HistoryTreeItemIn.IsValid() && HistoryTreeItemIn->RevisionListItem.IsValid())
	{
		TSharedPtr<FHistoryRevisionListViewItem> RevisionListItem = HistoryTreeItemIn->RevisionListItem;

		// fill out the revision info
		RevisionInfoOut.Revision = RevisionListItem->Revision;
		RevisionInfoOut.Changelist = RevisionListItem->ChangelistNumber;
		RevisionInfoOut.Date = RevisionListItem->Date;
	}
}

/**
 * Takes a array of FHistoryTreeItems and determines if the entries can all be diffed against each other.
 * 
 * @param  SelectedItems	A array of FHistoryTreeItems that you want to diff against each other.
 * @param  ErrorTextOut		Text explaining why the selected items cannot be diffed (only valid if the return value was false).
 * @return True if the selected items could be diffed against each other, false if not.
 */
static bool CanDiffSelectedItems(TArray< TSharedPtr<FHistoryTreeItem> > const& SelectedItems, FText& ErrorTextOut)
{
	bool bCanDiffSelected = false;

	if (SelectedItems.Num() > 2)
	{
		ErrorTextOut = NSLOCTEXT("SourceControlHistory", "TooManyToDiff", "Cannot diff more than two revisions.");
	}
	else if (SelectedItems.Num() < 2)
	{
		ErrorTextOut = NSLOCTEXT("SourceControlHistory", "NotEnoughToDiff", "Need to select two revisions in order to compare one against the other.");
	}
	else 
	{
		TSharedPtr<FHistoryTreeItem> FirstSelection  = SelectedItems[0];
		TSharedPtr<FHistoryTreeItem> SecondSelection = SelectedItems[1];

		if (!FirstSelection.IsValid() || !SecondSelection.IsValid())
		{
			ErrorTextOut = NSLOCTEXT("SourceControlHistory", "InvalidSelection", "Invalid revisions selected.");
		}
		else if (FirstSelection == SecondSelection)
		{
			ErrorTextOut = NSLOCTEXT("SourceControlHistory", "CannotDiffWithSelf", "You cannot diff a revision against itself.");
		}
		else if (!FirstSelection->Parent.IsValid() && FirstSelection->RevisionListItem.IsValid() && FirstSelection->Children[0] == SecondSelection)
		{
			ErrorTextOut = NSLOCTEXT("SourceControlHistory", "CannotDiffWithSelf", "You cannot diff a revision against itself.");
		}
		else if (!SecondSelection->Parent.IsValid() && FirstSelection->RevisionListItem.IsValid() && SecondSelection->Children[0] == FirstSelection)
		{
			ErrorTextOut = NSLOCTEXT("SourceControlHistory", "CannotDiffWithSelf", "You cannot diff a revision against itself.");
		}
		else
		{
			// @TODO make sure the two selections match type (calling GetAssetRevisionObject() to compare class types is too slow)
			bCanDiffSelected = true;
		}
	}

	return bCanDiffSelected;
};

/**
 * Takes two FHistoryTreeItems and attempts to diff them against each other (bringing up the diff window).
 * 
 * @param  FirstSelection	The first item you want to diff.
 * @param  SecondSelection	The second item you want to diff.
 * @return True if a diff was performed, false if not.
 */
static bool DiffHistoryItems(TSharedPtr<FHistoryTreeItem> const FirstSelection, TSharedPtr<FHistoryTreeItem> const SecondSelection)
{
	bool bDiffPerformed = false;

	if (FirstSelection.IsValid() && SecondSelection.IsValid())
	{
		TSharedPtr<FHistoryFileListViewItem> FirstSelectionFileItem = GetFileListItem(FirstSelection);
		TSharedPtr<FHistoryFileListViewItem> SecondSelectionFileItem = GetFileListItem(SecondSelection);

		// we want to make sure the two selections are presented in a sensible order 
		UObject* LeftDiffAsset = NULL;
		FRevisionInfo LeftVersionInfo;
		UObject* RightDiffAsset = NULL;
		FRevisionInfo RightVersionInfo;

		bool bIsForSingleAsset = (FirstSelectionFileItem == SecondSelectionFileItem);
		// if we're comparing two revisions for one asset
		if (bIsForSingleAsset) 
		{
			bool bFirstSelectionIsCurrentVersion = FirstSelection->FileListItem.IsValid();
			bool bSecondSelectionIsCurrentVersion = SecondSelection->FileListItem.IsValid();

			// the second selection is the newer revision iff the first isn't the current working version, and
			// it's either the current working version itself, or a newer revision
			bool bSecondSelectionIsNewer = !bFirstSelectionIsCurrentVersion && 
				(bSecondSelectionIsCurrentVersion ||(SecondSelection->RevisionListItem->Date > FirstSelection->RevisionListItem->Date));

			if (bSecondSelectionIsNewer)
			{
				RightDiffAsset = GetAssetRevisionObject(SecondSelection);
				GetRevisionInfo(SecondSelection, RightVersionInfo);
				LeftDiffAsset = GetAssetRevisionObject(FirstSelection);
				GetRevisionInfo(FirstSelection, LeftVersionInfo);
			}
			else 
			{
				LeftDiffAsset = GetAssetRevisionObject(SecondSelection);
				GetRevisionInfo(SecondSelection, LeftVersionInfo);
				RightDiffAsset = GetAssetRevisionObject(FirstSelection);
				GetRevisionInfo(FirstSelection, RightVersionInfo);
			}
		}
		else // else, we're comparing revisions from two separate assets
		{
			// keep them in selection order (left to right)
			LeftDiffAsset = GetAssetRevisionObject(FirstSelection);
			GetRevisionInfo(FirstSelection, LeftVersionInfo);
			RightDiffAsset = GetAssetRevisionObject(SecondSelection);
			GetRevisionInfo(SecondSelection, RightVersionInfo);
		}

		// if we have an asset object for both selections
		if ((LeftDiffAsset != NULL) && (RightDiffAsset != NULL))
		{
			FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
			AssetToolsModule.Get().DiffAssets(LeftDiffAsset, RightDiffAsset, LeftVersionInfo, RightVersionInfo);

			bDiffPerformed = true;
		}
	}
	
	return bDiffPerformed;
}

/**
* A FDragDropOperation that represents dragging a source-control history tree item around.
*/
class FSourceControlHistoryRowDragDropOp : public FDragDropOperation
{
private:
	/**
	 * Stubbed private constructor (in order to force use of the static New() method).
	 */
	FSourceControlHistoryRowDragDropOp()
		: PendingDropAction(EDropAction::None)
	{
	}

public:
	/**
	 * Allocates and registers a new FSourceControlHistoryRowDragDropOp for use.
	 * 
	 * @return The newly allocated (and registered) instance.
	 */
	static TSharedRef<FSourceControlHistoryRowDragDropOp> New()
	{
		TSharedPtr<FSourceControlHistoryRowDragDropOp> NewOperation = MakeShareable(new FSourceControlHistoryRowDragDropOp);
		NewOperation->Construct();

		return NewOperation.ToSharedRef();
	}

	DRAG_DROP_OPERATOR_TYPE(FSourceControlHistoryRowDragDropOp, FDragDropOperation)
	
	struct EDropAction
	{
		enum Type
		{
			None,
			Diff,
		};
	};
	/** An enum value detailing what operation is queued to happen (if this item is dropped) */
	EDropAction::Type PendingDropAction;

	/** The items that this operation is conceptually dragging around */
	TArray< TSharedPtr<FHistoryTreeItem> > SelectedItems;

	/** Text to display with the widget being dragged around */
	FText HoverText;

	/**
	 * Creates the visual widget that you drag around (to help visualize the drag/drop operation.
	 * 
	 * @return A new slate widget representing the dragged item
	 */
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Graph.ConnectorFeedback.Border"))
			[				
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0,0,3,0)
					[
						SNew(SImage)
							.Image(this, &FSourceControlHistoryRowDragDropOp::GetIcon)
					]
				+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock) 
							.Text(this, &FSourceControlHistoryRowDragDropOp::GetHoverText)
					]
			];
	}

	/**
	 * Easy accessor function for getting at the HoverText variable (provides a 
	 * default one if HoverText is empty). 
	 * 
	 * @return HoverText if it's not empty, otherwise: a default string describing that dropping would result in nothing.
	 */
	FText GetHoverText() const
	{
		return !HoverText.IsEmpty()
			? HoverText
			: NSLOCTEXT("SourceControlHistory", "DropActionToolTip_InvalidDropTarget", "Cannot drop here.");
	}

	/**
	 * Returns a icon brush corresponding to this operation's pending drop action. 
	 * 
	 * @return An 'error' icon if there is no pending action, otherwise an 'OK' icon.
	 */
	FSlateBrush const* GetIcon() const
	{
		return PendingDropAction != EDropAction::None
			? FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"))
			: FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));
	}
};

/**
 * Class used to construct the ordered row content for revision data
 */
class SHistoryRevisionListRowContent : public SMultiColumnTableRow< TSharedPtr<FHistoryTreeItem> >
{
public:

	SLATE_BEGIN_ARGS( SHistoryRevisionListRowContent )
		: _RevisionListItem()
	{}
		SLATE_EVENT( FOnDragDetected, OnDragDetected )
		SLATE_EVENT( FOnTableRowDragEnter, OnDragEnter )
		SLATE_EVENT( FOnTableRowDragLeave, OnDragLeave )
		SLATE_EVENT( FOnTableRowDrop,      OnDrop )

		SLATE_ARGUMENT( TSharedPtr<FHistoryRevisionListViewItem>, RevisionListItem )

		/** Whether we should display the expander for this item as it has children */
		SLATE_ARGUMENT( bool, HasChildren )

	SLATE_END_ARGS()	

	/**
	 * Construct the widget
	 *
	 * @param InArgs   A declaration from which to construct the widget
	 */
	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView )
	{
		RevisionListItem = InArgs._RevisionListItem;
		check(RevisionListItem.IsValid());

		bHasChildren = InArgs._HasChildren;

		SMultiColumnTableRow< TSharedPtr<FHistoryTreeItem> >::Construct( 
			FSuperRowType::FArguments()
				.OnDragDetected(InArgs._OnDragDetected) 
				.OnDragEnter(InArgs._OnDragEnter)
				.OnDragLeave(InArgs._OnDragLeave)
				.OnDrop(InArgs._OnDrop), 
			InOwnerTableView 
		);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override
	{
		check(RevisionListItem.IsValid());

		if ( ColumnName == TEXT("Revision") )
		{
			
			FString SCCAction = RevisionListItem->Action;
			FName ResourceKey;
			if (SCCAction == FString(TEXT("add")))
			{
				ResourceKey = TEXT("SourceControl.Add");
			}
			else if (SCCAction == FString(TEXT("edit")))
			{
				ResourceKey = TEXT("SourceControl.Edit");
			}
			else if (SCCAction == FString(TEXT("delete")))
			{
				ResourceKey = TEXT("SourceControl.Delete");
			}
			else if (SCCAction == FString(TEXT("branch")))
			{
				ResourceKey = TEXT("SourceControl.Branch");
			}
			else if (SCCAction == FString(TEXT("integrate")))
			{
				ResourceKey = TEXT("SourceControl.Integrate");
			}
			else
			{
				ResourceKey = TEXT("SourceControl.Edit");
			}

			// Rows in a tree need to show an SExpanderArrow (it also indents!) to give the appearance of being a tree.
			return
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right) 
				.VAlign(VAlign_Fill)
				[
					SNew(SExpanderArrow, SharedThis(this) )
					.Visibility(this, &SHistoryRevisionListRowContent::GetExpanderVisibility)
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(10,0,10,0)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(ResourceKey))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text( FText::FromString(RevisionListItem->Revision) )
				];	
		}
		else if (ColumnName == TEXT("Changelist"))
		{
			return
				SNew(STextBlock) 
				.Text( FText::AsNumber( RevisionListItem->ChangelistNumber, NULL, FInternationalization::Get().GetInvariantCulture() ) ) ;
		}
		else if (ColumnName == TEXT("Date"))
		{
			return
				SNew(STextBlock)
				.Text( RevisionListItem->Date > FDateTime::MinValue() == 0 ? FText() : FText::AsDateTime( RevisionListItem->Date ) );
		}
		else if (ColumnName == TEXT("UserName"))
		{
			return
				SNew(STextBlock)
				.Text( FText::FromString( RevisionListItem->UserName ) );
		}
		else if (ColumnName == TEXT("Description"))
		{
			return
				SNew(STextBlock)
				.Text(SSourceControlCommon::GetSingleLineChangelistDescription(FText::FromString(RevisionListItem->Description)))
				.ToolTipText(FText::FromString(RevisionListItem->Description))
				.OverflowPolicy(ETextOverflowPolicy::Ellipsis);
		}
		else
		{
			return
				SNew(STextBlock)
				. Text( FText::Format( NSLOCTEXT("SourceControlHistory", "UnsupportedColumn", "Unsupported Column: {0}"), FText::FromName( ColumnName ) ) );
		}
	}
	
	EVisibility GetExpanderVisibility() const
	{
		return bHasChildren ? EVisibility::Visible : EVisibility::Collapsed;
	}

private:
	TSharedPtr<FHistoryRevisionListViewItem> RevisionListItem;

	/** Whether we should display the expander for this item as it has children */
	bool bHasChildren;
};

/** Panel designed to display the revision history of a package */
class SSourceControlHistoryWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SSourceControlHistoryWidget )
		: _ParentWindow()
		, _SourceControlStates()
	{}

	SLATE_ATTRIBUTE( TSharedPtr<SWindow>, ParentWindow )
	SLATE_ATTRIBUTE( TArray< FSourceControlStateRef >, SourceControlStates)

	SLATE_END_ARGS()
		
	const FName SourceControlHistoryContextMenu = TEXT("RevisionControl.History.ContextMenu");

	SSourceControlHistoryWidget()
	{
	}

	static void CreateDiffMenu(UToolMenu* InToolMenu);

	void Construct( const FArguments& InArgs )
	{	
		if (!UToolMenus::Get()->IsMenuRegistered(SourceControlHistoryContextMenu))
		{
			UToolMenu* ContextMenu = UToolMenus::Get()->RegisterMenu(SourceControlHistoryContextMenu);
			ContextMenu->bShouldCloseWindowAfterMenuSelection = true;

			ContextMenu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateStatic(&SSourceControlHistoryWidget::CreateDiffMenu));
		}

		AddHistoryInfo(InArgs._SourceControlStates.Get());
		ParentWindow = InArgs._ParentWindow.Get();

		TSharedRef<SHeaderRow> HeaderRow = SNew(SHeaderRow);

		const bool bUsesChangelists = ISourceControlModule::Get().GetProvider().UsesChangelists();

		HeaderRow->AddColumn(SHeaderRow::FColumn::FArguments().ColumnId("Revision") .DefaultLabel(NSLOCTEXT("SourceControl.HistoryPanel.Header", "Revision", "Revision"))	.FillWidth(bUsesChangelists ? 100 : 200));
		if(bUsesChangelists)
		{
			HeaderRow->AddColumn(SHeaderRow::FColumn::FArguments().ColumnId("Changelist") .DefaultLabel(NSLOCTEXT("SourceControl.HistoryPanel.Header", "Changelist", "ChangeList"))	.FillWidth(100));
		}
		HeaderRow->AddColumn(SHeaderRow::FColumn::FArguments().ColumnId("Date") .DefaultLabel(NSLOCTEXT("SourceControl.HistoryPanel.Header", "Date", "Date Submitted"))			.FillWidth(250));
		HeaderRow->AddColumn(SHeaderRow::FColumn::FArguments().ColumnId("UserName") .DefaultLabel(NSLOCTEXT("SourceControl.HistoryPanel.Header", "UserName", "Submitted By"))		.FillWidth(200));
		HeaderRow->AddColumn(SHeaderRow::FColumn::FArguments().ColumnId("Description") .DefaultLabel(NSLOCTEXT("SourceControl.HistoryPanel.Header", "Description", "Description"))	.FillWidth(650));

		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.BorderBackgroundColor(FLinearColor(0.5f,0.5f,0.5f,1.f))
			[
				SNew(SSplitter)
				.Orientation(Orient_Vertical)
				+SSplitter::Slot()
				.Value(0.5f)
				[
					SNew(SBorder)
					[
						SNew(SBox)
						.WidthOverride(600)
						[
							SAssignNew( MainHistoryListView, SHistoryFileListType)
							.TreeItemsSource( &HistoryCollection )
							.SelectionMode(ESelectionMode::Multi)
							.OnSelectionChanged(this, &SSourceControlHistoryWidget::OnRevisionPropertyChanged)
							.OnGenerateRow( this, &SSourceControlHistoryWidget::OnGenerateRowForHistoryFileList )
							.OnGetChildren( this, &SSourceControlHistoryWidget::OnGetChildrenForHistoryFileList )
							.OnContextMenuOpening(this, &SSourceControlHistoryWidget::OnCreateContextMenu )
							.HeaderRow
							(
								HeaderRow
							)
						]
					]
				]

				+SSplitter::Slot()
				.Value(0.5f)
				[
					SAssignNew(AdditionalInfoItemsControl,SBorder)
					.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
					[
						GetAdditionalInfoItemsControlContent()				
					]
				]
			]
		];

		//expand the top level nodes...
		for (int32 i=0; i<HistoryCollection.Num(); i++)
		{
			MainHistoryListView->SetItemExpansion(HistoryCollection[i],true);
		}

		ParentWindow.Pin()->SetWidgetToFocusOnActivate(MainHistoryListView);
	}

private:

	/** Used to intercept Escape key press, and interpret it as a close event */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		// Pressing escape returns as if the user closed the window
		if (InKeyEvent.GetKey() == EKeys::Escape)
		{
			ParentWindow.Pin()->RequestDestroyWindow();
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	/**
	 * Constructs the "Additional Info" panel that displays specific revision info
	 */
	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	TSharedRef<SWidget> GetAdditionalInfoItemsControlContent()
	{
		const float Padding = 2.f;

		return
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(0.25f)
				[
					//Text Column
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.FillHeight(0.25f)
					.Padding(Padding)
					[
						SNew(STextBlock)
						.Text(NSLOCTEXT("SourceControl.HistoryPanel.Info", "Revision", "Revision:"))
					]
					+SVerticalBox::Slot()
					.FillHeight(0.25f)
					.Padding(Padding)
					[
						SNew(STextBlock)
						.Text(NSLOCTEXT("SourceControl.HistoryPanel.Info", "Date", "Date Submitted:"))
					]
					+SVerticalBox::Slot()
					.FillHeight(0.25f)
					.Padding(Padding)
					[
						SNew(STextBlock)
						.Text(NSLOCTEXT("SourceControl.HistoryPanel.Info", "SubmittedBy", "Submitted By:"))
					]
					+SVerticalBox::Slot()
					.FillHeight(0.25f)
					.Padding(Padding)
					[
						SNew(STextBlock)
						.Text(NSLOCTEXT("SourceControl.HistoryPanel.Info", "Action", "Action:"))
					]			
				]
				+SHorizontalBox::Slot()
				.FillWidth(0.25f)
				.Padding(20,0)
				[
					//Data column
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.FillHeight(0.25f)
					.Padding(Padding)
					[
						SNew(SEditableText)
						.IsReadOnly(true)
						.Text(this, &SSourceControlHistoryWidget::GetRevisionNumber)	
					]
					+SVerticalBox::Slot()
					.FillHeight(0.25f)
					.Padding(Padding)
					[
						SNew(SEditableText)
						.IsReadOnly(true)
						.Text(this, &SSourceControlHistoryWidget::GetDate)	
					]
					+SVerticalBox::Slot()
					.FillHeight(0.25f)
					.Padding(Padding)
					[
						SNew(SEditableText)
						.IsReadOnly(true)
						.Text(this, &SSourceControlHistoryWidget::GetUserName)	
					]
					+SVerticalBox::Slot()
					.FillHeight(0.25f)
					.Padding(Padding)
					[
						SNew(SEditableText)
						.IsReadOnly(true)
						.Text(this, &SSourceControlHistoryWidget::GetAction)	
					]
				]
				+SHorizontalBox::Slot()
				.FillWidth(0.25f)
				.Padding(50,0)
				[
					//Text Column
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.FillHeight(0.25f)
					.Padding(Padding)
					[
						SNew(STextBlock)
						.Text(NSLOCTEXT("SourceControl.HistoryPanel.Info", "Changelist", "Changelist:"))
						.Visibility(ISourceControlModule::Get().GetProvider().UsesChangelists() ? EVisibility::SelfHitTestInvisible : EVisibility::Hidden)
					]
					+SVerticalBox::Slot()
					.FillHeight(0.25f)
					.Padding(Padding)
					[
						SNew(STextBlock)
						.Text(NSLOCTEXT("SourceControl.HistoryPanel.Info", "Workspace", "Workspace:"))
					]
					+SVerticalBox::Slot()
					.FillHeight(0.25f)
					.Padding(Padding)
					[
						SNew(STextBlock)
						.Text(NSLOCTEXT("SourceControl.HistoryPanel.Info", "FileSize", "File Size:"))
					]
					+SVerticalBox::Slot()
					.FillHeight(0.25f)
					.Padding(Padding)
					[
						SNew(STextBlock)
						.Text(NSLOCTEXT("SourceControl.HistoryPanel.Info", "BranchedFrom", "Branched From:"))
					]
				]
				+SHorizontalBox::Slot()
				.FillWidth(0.25f)
				.Padding(20,0)
				[
					//Data column
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.FillHeight(0.25f)
					.Padding(Padding)
					[
						SNew(SEditableText)
						.IsReadOnly(true)
						.Text(this, &SSourceControlHistoryWidget::GetChangelistNumber)	
					]
					+SVerticalBox::Slot()
					.FillHeight(0.25f)
					.Padding(Padding)
					[
						SNew(SEditableText)
						.IsReadOnly(true)
						.Text(this, &SSourceControlHistoryWidget::GetClientSpec)	
					]
					+SVerticalBox::Slot()
					.FillHeight(0.25f)
					.Padding(Padding)
					[
						SNew(SEditableText)
						.IsReadOnly(true)
						.Text(this, &SSourceControlHistoryWidget::GetFileSize)	
					]
					+SVerticalBox::Slot()
					.FillHeight(0.25f)
					.Padding(Padding)
					[
						SNew(SEditableText)
						.IsReadOnly(true)
						.Text(this, &SSourceControlHistoryWidget::GetBranchedFrom)	
					]
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(Padding, 10, Padding, 5)
			[
				SNew(STextBlock)
				.Text(NSLOCTEXT("SourceControl.HistoryPanel.Info", "Description", "Description:"))
			]
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SBorder)
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.Padding(5)
					[
						SNew(SMultiLineEditableTextBox)
						.IsReadOnly(true)
						.AutoWrapText(true)
						.Text(this, &SSourceControlHistoryWidget::GetDescription)
					]
				]
			]
		;
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

	/** Get the last selected revision's revision number */
	FText GetRevisionNumber() const
	{
		if(LastSelectedRevisionItem.IsValid())
		{
			return FText::FromString(LastSelectedRevisionItem.Pin()->Revision);
		}
		return FText::GetEmpty();
	}

	/** Get the last selected revision's date */
	FText GetDate() const
	{
		if(LastSelectedRevisionItem.IsValid() && LastSelectedRevisionItem.Pin()->Date != 0)
		{
			return FText::AsDateTime(LastSelectedRevisionItem.Pin()->Date);
		}
		return FText::GetEmpty();
	}

	/** Get the last selected revision's username */
	FText GetUserName() const
	{
		if(LastSelectedRevisionItem.IsValid())
		{
			return FText::FromString(LastSelectedRevisionItem.Pin()->UserName);
		}
		return FText::GetEmpty();
	}

	/** Get the last selected revision's revision number */
	FText GetAction() const
	{
		if(LastSelectedRevisionItem.IsValid())
		{
			return FText::FromString(LastSelectedRevisionItem.Pin()->Action);
		}
		return FText::GetEmpty();
	}

	/** Get the last selected revision's changelist number */
	FText GetChangelistNumber() const
	{
		static const bool bUsesChangelists = ISourceControlModule::Get().GetProvider().UsesChangelists();
		if(LastSelectedRevisionItem.IsValid() && bUsesChangelists)
		{
			// don't group the CL# as Perforce doesn't display it that way
			return FText::AsNumber(LastSelectedRevisionItem.Pin()->ChangelistNumber, &FNumberFormattingOptions::DefaultNoGrouping());
		}
		return FText::GetEmpty();
	}

	/** Get the last selected revision's client spec */
	FText GetClientSpec() const
	{
		if(LastSelectedRevisionItem.IsValid())
		{
			return FText::FromString(LastSelectedRevisionItem.Pin()->ClientSpec);
		}
		return FText::GetEmpty();
	}

	/** Get the last selected revision's file size */
	FText GetFileSize() const
	{
		if(LastSelectedRevisionItem.IsValid())
		{
			static const FNumberFormattingOptions FileSizeFormatOptions = FNumberFormattingOptions()
				.SetMinimumFractionalDigits(1)
				.SetMaximumFractionalDigits(1);
			return FText::AsMemory(LastSelectedRevisionItem.Pin()->FileSize, &FileSizeFormatOptions);
		}
		return FText::GetEmpty();
	}

	/** Get the last selected revision's description */
	FText GetDescription() const
	{
		if(LastSelectedRevisionItem.IsValid())
		{
			return FText::FromString(LastSelectedRevisionItem.Pin()->Description);
		}
		return FText::GetEmpty();
	}

	/** Get the last selected revision's description */
	FText GetBranchedFrom() const
	{
		if(LastSelectedRevisionItem.IsValid())
		{
			return FText::FromString(LastSelectedRevisionItem.Pin()->BranchSource);
		}
		return FText::GetEmpty();
	}

	/**
	 * Generates the content of each row, displaying a the File or Revision data for its corresponding type
	 */
	TSharedRef<ITableRow> OnGenerateRowForHistoryFileList( TSharedPtr<FHistoryTreeItem> TreeItemPtr, const TSharedRef<STableViewBase>& OwnerTable )
	{
		TSharedPtr<SWidget> RowContent;
		if (TreeItemPtr->FileListItem.IsValid())
		{
			TSharedPtr<FHistoryFileListViewItem> FileListItem = TreeItemPtr->FileListItem;
			FString FileName = FileListItem->FileName;
			FPaths::MakePlatformFilename(FileName);

			return
				SNew(STableRow< TSharedPtr<FName> >, OwnerTable)
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(5)
					[
						SNew( STextBlock )
						.Font( FAppStyle::GetFontStyle( TEXT("BoldFont") ))
						.Text( FText::FromString(MoveTemp(FileName)) )
					]
				]
				.OnDragDetected(this, &SSourceControlHistoryWidget::OnRowDragDetected)
				.OnDragEnter(this, &SSourceControlHistoryWidget::OnRowDragEnter, TreeItemPtr)
				.OnDragLeave(this, &SSourceControlHistoryWidget::OnRowDragLeave)
				.OnDrop(this, &SSourceControlHistoryWidget::OnRowDrop, TreeItemPtr);
		}
		else if (TreeItemPtr->RevisionListItem.IsValid())
		{
			TSharedPtr<FHistoryRevisionListViewItem> RevisionListItem = TreeItemPtr->RevisionListItem;

			return
				SNew(SHistoryRevisionListRowContent, OwnerTable)
				.RevisionListItem(RevisionListItem)
				.OnDragDetected(this, &SSourceControlHistoryWidget::OnRowDragDetected)
				.OnDragEnter(this, &SSourceControlHistoryWidget::OnRowDragEnter, TreeItemPtr)
				.OnDragLeave(this, &SSourceControlHistoryWidget::OnRowDragLeave)
				.OnDrop(this, &SSourceControlHistoryWidget::OnRowDrop, TreeItemPtr)
				.HasChildren(TreeItemPtr->Children.Num() > 0);
		}
		
		//we should never get here...
		return 
			SNew(STableRow< TSharedPtr<FName> >, OwnerTable)
			[
				SNew( STextBlock )
				.Text( NSLOCTEXT("SourceControlHistory", "ErrorMessage", "---ERROR---") )
			];	
	}

	/**
	 * Fill out the tree structure with the SSC data
	 */
	void AddHistoryInfo( const TArray< FSourceControlStateRef >& InStates )
	{
		// Construct a new observable collection to serve as the items source for the main list view. It will contain each history file item.
		for( TArray< FSourceControlStateRef >::TConstIterator Iter(InStates); Iter; Iter++)
		{
			FSourceControlStateRef SourceControlState = *Iter;
			TSharedPtr<FHistoryTreeItem> FileItem = MakeShareable(new FHistoryTreeItem());
			FileItem->FileListItem = MakeShareable(new FHistoryFileListViewItem( SourceControlState->GetFilename() ));

			// Add each file revision
			for ( int32 HistoryIndex = 0; HistoryIndex < SourceControlState->GetHistorySize(); HistoryIndex++ )
			{
				TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> Revision = SourceControlState->GetHistoryItem(HistoryIndex);
				check(Revision.IsValid());
				TSharedPtr<FHistoryTreeItem> RevisionItem = MakeShareable(new FHistoryTreeItem());
				RevisionItem->RevisionListItem = MakeShareable(new FHistoryRevisionListViewItem( Revision.ToSharedRef() ));
				FileItem->Children.Add(RevisionItem);
				RevisionItem->Parent = FileItem;

				// File Items other than the current workspace version masquarade as the latest version of that file
				if (HistoryIndex == 0 && HistoryCollection.Num() > 0)
				{
					FileItem->RevisionListItem = RevisionItem->RevisionListItem;
				}

				// add branch items if we have one
				if(Revision->GetBranchSource().IsValid())
				{
					TSharedPtr<FHistoryTreeItem> BranchFileItem = MakeShareable(new FHistoryTreeItem());

					const FString BranchRevisionName = FString::Printf(TEXT("%s #%d"), *Revision->GetBranchSource()->GetFilename(), Revision->GetBranchSource()->GetRevisionNumber());
					BranchFileItem->FileListItem = MakeShareable(new FHistoryFileListViewItem( BranchRevisionName ));
					RevisionItem->Children.Add(BranchFileItem);
					BranchFileItem->Parent = RevisionItem;
				}
			}

			HistoryCollection.Add(FileItem);
		}
	}

	/**
	 * Callback returns the revision history (Children) nodes for the file (InItem) node
	 */
	void OnGetChildrenForHistoryFileList( TSharedPtr< FHistoryTreeItem > InItem, TArray< TSharedPtr<FHistoryTreeItem> >& OutChildren )
	{
		OutChildren = InItem->Children;
	}

	/**
	 * Called whenever the IsSelected property on a MHistoryRevisionListViewItem changes. Used to specify the last selected revision item.
	 *
	 * @param	Owner	Object which triggered the event
	 * @param	Args	Event arguments for the property change
	 */
	void OnRevisionPropertyChanged(TSharedPtr<FHistoryTreeItem> Item, ESelectInfo::Type SelectInfo)
	{
		LastSelectedRevisionItem.Reset();
		if (Item.IsValid())
		{
			if (Item->RevisionListItem.IsValid())
			{
				LastSelectedRevisionItem = Item->RevisionListItem;
			}
			else if (Item->Children.Num() > 0 && Item->Children[0]->RevisionListItem.IsValid())
			{
				LastSelectedRevisionItem = Item->Children[0]->RevisionListItem;	
			}
		}
		

		AdditionalInfoItemsControl->SetContent(GetAdditionalInfoItemsControlContent());
	}

	/** Called to create a context menu when right-clicking on a history item */
	TSharedPtr< SWidget > OnCreateContextMenu()
	{
		FToolMenuContext Context;
		USourceControlHistoryWidgetContext* SourceControlHistoryWidgetContext = NewObject<USourceControlHistoryWidgetContext>();
		SourceControlHistoryWidgetContext->HistoryWidget = SharedThis(this);

		for (TSharedPtr<FHistoryTreeItem> Item : MainHistoryListView->GetSelectedItems())
		{
			if (Item->Parent.IsValid())
			{
				TSharedPtr<FHistoryTreeItem> Parent = Item->Parent.Pin();

				USourceControlHistoryWidgetContext::SelectedItem ItemInfo;
				if (Parent->FileListItem)
				{
					ItemInfo.FileName = Parent->FileListItem->FileName;
					ItemInfo.Revision = Item->RevisionListItem->Revision;

					SourceControlHistoryWidgetContext->GetSelectedItems().Add(ItemInfo);
				}
			}
		}
	
		Context.AddObject(SourceControlHistoryWidgetContext);
		if (UToolMenu* GeneratedContextMenu = UToolMenus::Get()->GenerateMenu(SourceControlHistoryContextMenu, Context))
		{
			return UToolMenus::Get()->GenerateWidget(GeneratedContextMenu);
		}

		return SNullWidget::NullWidget;
	}

	/** See if we should enabled the 'diff against previous' option */
	bool CanDiffAgainstPreviousRev() const
	{
		TArray< TSharedPtr<FHistoryTreeItem> > SelectedRevs = MainHistoryListView->GetSelectedItems();

		// Only allow option if we selected one item
		if (SelectedRevs.Num() == 1 && SelectedRevs[0].IsValid())
		{
			TSharedPtr<FHistoryTreeItem> SelectedItem = SelectedRevs[0];

			if (SelectedItem->Parent.IsValid())
			{
				// If it is a revision then allow diffing against previous revision unless it is the original revision
				TSharedPtr<FHistoryTreeItem> FileItem = SelectedItem->Parent.Pin();
				check(FileItem.IsValid());

				// If we've got the expanded RevisionList entry rather than the FileList entry
				// move the Selected and FileItem up one level in the hierarchy
				if (FileItem->FileListItem == nullptr)
				{
					SelectedItem = FileItem;
					FileItem = FileItem->Parent.Pin();
					check(FileItem.IsValid())
				}

				int32 RevIndex = FileItem->Children.Find(SelectedItem);
				check(RevIndex != INDEX_NONE);
				if (RevIndex == FileItem->Children.Num()-1) // If oldest revision of this file
				{
					int32 FileIndex = HistoryCollection.Find(FileItem);
					check(FileIndex != INDEX_NONE); 
					return (FileIndex < HistoryCollection.Num()-1);
				}
			}
			else
			{
				// If it is a file item then allow diffing against previous revision unless it is the last
				// file item and it only has one child
				const int32 FileIndex = HistoryCollection.Find(SelectedItem);
				if (FileIndex > 0 && FileIndex == HistoryCollection.Num() - 1)
				{
					return (SelectedItem->Children.Num() > 1);
				}
			}

			return true;
		}

		return false;
	}

	/** Try and perform a diff between the selected revision and the previous one */
	void OnDiffAgainstPreviousRev()
	{
		TArray< TSharedPtr<FHistoryTreeItem> > SelectedRevs = MainHistoryListView->GetSelectedItems();

		if(SelectedRevs.Num() > 0 && SelectedRevs[0].IsValid())
		{
			FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));

			TSharedPtr<FHistoryTreeItem> SelectedItem = SelectedRevs[0];

			// This might be the expanded RevisionList item, if so we need to push up one level in the hierarchy
			if (!SelectedItem->RevisionListItem.IsValid())
			{
				TSharedPtr<FHistoryTreeItem> ParentItem = SelectedItem->Parent.Pin();
				if (ParentItem.IsValid() && ParentItem->RevisionListItem.IsValid())
				{
					SelectedItem = ParentItem;
				}
			}

			UObject* SelectedAsset = GetAssetRevisionObject(SelectedItem);

			if (SelectedItem->RevisionListItem.IsValid())
			{
				TSharedPtr<FHistoryTreeItem> FileItem;
				int32 RevIndex;
				
				if (SelectedItem->Parent.IsValid())
				{
					FileItem = SelectedItem->Parent.Pin();
					check(FileItem.IsValid());

					// First, find index of selected revision in its parent file item
					// NB. 0 is newest, increasing index means older
					RevIndex = FileItem->Children.Find(SelectedItem);
					check(RevIndex != INDEX_NONE);
				}
				else
				{
					// If the selected item doesn't have a parent, this is a file item
					// masquerading as the most recent revision
					FileItem = SelectedItem;
					RevIndex = 0;
				}

				// Now we need to find previous revision
				TSharedPtr<FHistoryTreeItem> PreRevisionItem;

				if(RevIndex == FileItem->Children.Num()-1) // If oldest revision of this file
				{
					// .. see if we have an older file
					int32 FileIndex = HistoryCollection.Find(FileItem);
					check(FileIndex != INDEX_NONE);
					// Do nothing if we selected the newest revision of the newest file...
					if(FileIndex < HistoryCollection.Num()-1)
					{
						// Previous revision is a different file, so get the newest revision of the older file
						TSharedPtr<FHistoryTreeItem> PrevFileItem = HistoryCollection[FileIndex+1];
						check(PrevFileItem.IsValid());
						if(PrevFileItem->Children.Num() > 0)
						{
							PreRevisionItem = PrevFileItem->Children[0];
						}
					}
				}
				else
				{
					// Not the oldest revision of this file, grab the older entry and get revision number
					PreRevisionItem = FileItem->Children[RevIndex+1];
				}

				UObject* PreviousAsset = GetAssetRevisionObject(PreRevisionItem);

				if ((SelectedAsset != NULL) && (PreviousAsset != NULL))
				{
					FRevisionInfo OldRevisionInfo;
					GetRevisionInfo(PreRevisionItem, OldRevisionInfo);
					FRevisionInfo NewRevisionInfo;
					GetRevisionInfo(SelectedItem, NewRevisionInfo);

					AssetToolsModule.Get().DiffAssets(PreviousAsset, SelectedAsset, OldRevisionInfo, NewRevisionInfo);
				}
				else
				{
					FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("SourceControl.HistoryWindow", "UnableToLoadAssets", "Unable to load assets to diff. Content may no longer be supported?"));
				}
			}
			else if (SelectedAsset != NULL)
			{
				// this should be a file list-item (representing the current working version)
				check(SelectedItem->FileListItem.IsValid());

				FString const AssetName = SelectedAsset->GetName();
				FString PackageName;
				if (FPackageName::TryConvertFilenameToLongPackageName(SelectedItem->FileListItem->FileName, /*out*/ PackageName))
				{
					AssetToolsModule.Get().DiffAgainstDepot(SelectedAsset, PackageName, AssetName);
				}
			}
		}
	}

	/** See if we should enabled the 'diff against workspace' option */
	bool CanDiffAgainstWorkspace() const
	{
		// Only allow option if we selected one item and it is not the current workspace row
		TArray< TSharedPtr<FHistoryTreeItem> > SelectedRevs = MainHistoryListView->GetSelectedItems();
		return (SelectedRevs.Num() == 1 && SelectedRevs[0].IsValid() && HistoryCollection[0] != SelectedRevs[0]);
	}

	/** Try and perform a diff between the selected revision and the current version of the asset */
	void OnDiffAgainstWorkspace()
	{
		TArray< TSharedPtr<FHistoryTreeItem> > SelectedRevs = MainHistoryListView->GetSelectedItems();

		if (SelectedRevs.Num() > 0 && SelectedRevs[0].IsValid())
		{
			FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));

			TSharedPtr<FHistoryTreeItem> SelectedItem = SelectedRevs[0];

			// This might be the expanded RevisionList item, if so we need to push up one level in the hierarchy
			if (!SelectedItem->RevisionListItem.IsValid())
			{
				TSharedPtr<FHistoryTreeItem> ParentItem = SelectedItem->Parent.Pin();
				if (ParentItem.IsValid() && ParentItem->RevisionListItem.IsValid())
				{
					SelectedItem = ParentItem;
				}
			}

			UObject* SelectedAsset = GetAssetRevisionObject(SelectedItem);

			if (SelectedItem->RevisionListItem.IsValid())
			{
				UObject* CurrentAsset = (HistoryCollection.Num() > 0 ? GetAssetRevisionObject(HistoryCollection[0]) : nullptr);

				if ((SelectedAsset != NULL) && (CurrentAsset != NULL))
				{
					FRevisionInfo OldRevisionInfo;
					GetRevisionInfo(SelectedItem, OldRevisionInfo);
					FRevisionInfo NewRevisionInfo;
					NewRevisionInfo.Revision = TEXT("");

					AssetToolsModule.Get().DiffAssets(SelectedAsset, CurrentAsset, OldRevisionInfo, NewRevisionInfo);
				}
				else
				{
					FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("SourceControl.HistoryWindow", "UnableToLoadAssets", "Unable to load assets to diff. Content may no longer be supported?"));
				}
			}
			else if (SelectedAsset != NULL)
			{
				// this should be a file list-item (representing the current working version)
				check(SelectedItem->FileListItem.IsValid());

				FString const AssetName = SelectedAsset->GetName();
				FString PackageName;
				if (FPackageName::TryConvertFilenameToLongPackageName(SelectedItem->FileListItem->FileName, /*out*/ PackageName))
				{
					AssetToolsModule.Get().DiffAgainstDepot(SelectedAsset, PackageName, AssetName);
				}
			}
		}
	}

	/**
	 * Checks to see if the SourceControl provider supports diffing against depot.
	 *
	 * @return True if the SourceControl provider supports diffing, false if not.
	 */
	static bool CanDiff()
	{
		return ISourceControlModule::Get().GetProvider().AllowsDiffAgainstDepot();
	}

	/**
	 * Checks to see if the selected history-tree items can be diffed against each other.
	 * 
	 * @return True if the selected items can be diffed, false if not.
	 */
	bool CanDiffSelected() const
	{
		// throw away text so we can utilize a shared utility method
		FText ThrowAwayErrorText; 

		TArray< TSharedPtr<FHistoryTreeItem> > SelectedRevs = MainHistoryListView->GetSelectedItems();
		return CanDiffSelectedItems(SelectedRevs, ThrowAwayErrorText);
	}

	/**
	 * Takes the two selected history items and finds a UObject asset for each,
	 * then attempts to open a diff window to compare them.
	 */
	void OnDiffSelected() const
	{
		TArray< TSharedPtr<FHistoryTreeItem> > SelectedRevs = MainHistoryListView->GetSelectedItems();
		if (SelectedRevs.Num() >= 2)
		{
			if (!DiffHistoryItems(SelectedRevs[0], SelectedRevs[1]))
			{
				FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("SourceControl.HistoryWindow", "UnableToLoadAssets", "Unable to load assets to diff. Content may no longer be supported?"));
			}
		}
	}

	/** 
	* CanDiffSelected gives a dynamic response based on a number of factors, so CreateDiffSelectedMenu ensures we are checking on each menu open event and correctly adding the related menu item(s) for each case.
	*/
	void CreateDiffSelectedMenu(FToolMenuSection& InSection)
	{
		if (CanDiffSelected())
		{
			InSection.AddMenuEntry(
				TEXT("DiffSelected"),
				NSLOCTEXT("SourceControl.HistoryWindow.Menu", "DiffSelected", "Diff Selected"),
				NSLOCTEXT("SourceControl.HistoryWindow.Menu", "DiffSelectedTooltip", "Diff the two assets that you have selected."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(this, &SSourceControlHistoryWidget::OnDiffSelected)
				)
			);
		}
	}

	/**
	 * An event handler for mouse drag detection events. Intended to be used as a delegate for
	 * history tree rows. Creates a FSourceControlHistoryRowDragDropOp (if the drag was with 
	 * the left mouse-button) and assumes that all the selected items are the objects being dragged.
	 * 
	 * @param  MyGeometry	The geometry for the dragged widget.
	 * @param  MouseEvent	Describes the mouse drag action (from when the drag was detected).
	 * @return A reply detailing how this event was handled ("Unhandled" if the click was not a left-click).
	 */
	FReply OnRowDragDetected(FGeometry const& MyGeometry, FPointerEvent const& MouseEvent) const
	{
		if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton)) //can only drag when editing
		{
			TSharedRef<FSourceControlHistoryRowDragDropOp> DragOperation = FSourceControlHistoryRowDragDropOp::New();

			// assume that what we're dragging is what we have selected
			DragOperation->SelectedItems = MainHistoryListView->GetSelectedItems();
			check(DragOperation->SelectedItems.Num() > 0); 

			return FReply::Handled().BeginDragDrop(DragOperation);
		}
		else
		{
			return FReply::Unhandled();
		}
	}

	/**
	 * An event handler for mouse drag enter events. Intended to be used as a delegate for
	 * history tree rows. Only handles FSourceControlHistoryRowDragDropOp drag/drop operations.
	 * Updates the FSourceControlHistoryRowDragDropOp to reflect if a drop action is doable 
	 * (when over this item).
	 * 	
	 * @param  DragDropEvent	The drag/drop operation that triggered this handler.
	 * @param  HoveredItem		The history tree item that is conceptually being hovered over.
	 */
	void OnRowDragEnter(FDragDropEvent const& DragDropEvent, TSharedPtr<FHistoryTreeItem> const HoveredItem) const
	{
		TSharedPtr<FSourceControlHistoryRowDragDropOp> DragRowOp = DragDropEvent.GetOperationAs<FSourceControlHistoryRowDragDropOp>();
		if (DragRowOp.IsValid())
		{
			DragRowOp->PendingDropAction = FSourceControlHistoryRowDragDropOp::EDropAction::None;

			TArray< TSharedPtr<FHistoryTreeItem> > DiffingItems = DragRowOp->SelectedItems;
			check(HoveredItem.IsValid());
			DiffingItems.Add(HoveredItem);

			if (CanDiffSelectedItems(DiffingItems, DragRowOp->HoverText))
			{
				DragRowOp->PendingDropAction = FSourceControlHistoryRowDragDropOp::EDropAction::Diff;

				FText const RevisionFormatText  = NSLOCTEXT("SourceControlHistory", "Revision", "Revision {0}");
				FText const CurrentRevisionText = NSLOCTEXT("SourceControlHistory", "CurrentRevsion", "Current Revision");

				check(DragRowOp->SelectedItems.Num() > 0);
				TSharedPtr<FHistoryTreeItem> DraggedItem = DiffingItems[0];
				// set text identifying the dragged item's revision (current version vs. revision X)
				FText DraggedRevisionText = CurrentRevisionText;
				if (DraggedItem->RevisionListItem.IsValid())
				{
					DraggedRevisionText = FText::Format(RevisionFormatText, FText::FromString(DraggedItem->RevisionListItem->Revision));
				}

				// set text identifying the hovered item's revision (current version vs. revision X)
				FText HoveredRevisionText = CurrentRevisionText;
				if (HoveredItem->RevisionListItem.IsValid())
				{
					HoveredRevisionText = FText::Format(RevisionFormatText, FText::FromString(HoveredItem->RevisionListItem->Revision));
				}

				TSharedPtr<FHistoryFileListViewItem> const DraggedFileItem = GetFileListItem(DraggedItem);
				// convert DraggedRevisionText from the form "revision X" (or "the current version") to 
				// "revision X of <filename>"
				FString const DraggedFileName = FPaths::GetBaseFilename(DraggedFileItem->FileName);
				FText const NamedRevisionTextFormat = NSLOCTEXT("SourceControlHistory", "NamedRevision", "{0} ({1})");
				DraggedRevisionText = FText::Format(NamedRevisionTextFormat, FText::FromString(DraggedFileName), DraggedRevisionText);

				TSharedPtr<FHistoryFileListViewItem> const HoveredFileItem = GetFileListItem(HoveredItem);
				// if we're diffing two separate files against each other
				if (DraggedFileItem != HoveredFileItem)
				{
					// need to separately identify the hovered over item
					FString const HoveredFileName = FPaths::GetBaseFilename(HoveredFileItem->FileName);
					HoveredRevisionText = FText::Format(NamedRevisionTextFormat, FText::FromString(HoveredFileName), HoveredRevisionText);
				}

				FText const DropToDiffTextFormat =  NSLOCTEXT("SourceControlHistory", "DropToDiff", "Drop {0} to diff against: {1}.");
				DragRowOp->HoverText = FText::Format(DropToDiffTextFormat, DraggedRevisionText, HoveredRevisionText);
			}
		}
	}

	/**
	 * An event handler for mouse drag leave events. Intended to be used as a delegate for
	 * history tree rows. Only handles FSourceControlHistoryRowDragDropOp drag/drop operations.
	 * Clears any pending drop actions from the FSourceControlHistoryRowDragDropOp.
	 * 
	 * @param  DragDropEvent	The drag/drop operation that triggered this handler.
	 */
	void OnRowDragLeave(FDragDropEvent const& DragDropEvent) const
	{
		TSharedPtr<FSourceControlHistoryRowDragDropOp> DragRowOp = DragDropEvent.GetOperationAs<FSourceControlHistoryRowDragDropOp>();
		if (DragRowOp.IsValid())
		{
			DragRowOp->HoverText = FText::GetEmpty();
			DragRowOp->PendingDropAction = FSourceControlHistoryRowDragDropOp::EDropAction::None;
		}
	}

	/**
	 * An event handler for mouse drop events. Intended to be used as a delegate for
	 * history tree rows. Only handles FSourceControlHistoryRowDragDropOp drag/drop operations.
	 * Executes the pending FSourceControlHistoryRowDragDropOp action (diff, etc.)
	 * 	
	 * @param  DragDropEvent	The drag/drop operation that triggered this handler.
	 * @param  HoveredItem		The history tree item that is conceptually being dropped on to.
	 * @return A reply detailing how this event was handled ("Unhandled" if the operation was not a FSourceControlHistoryRowDragDropOp).
	 */
	FReply OnRowDrop(FDragDropEvent const& DragDropEvent, TSharedPtr<FHistoryTreeItem> const HoveredItem) const
	{
		TSharedPtr<FSourceControlHistoryRowDragDropOp> DragRowOp = DragDropEvent.GetOperationAs<FSourceControlHistoryRowDragDropOp>();
		if (DragRowOp.IsValid())
		{
			if (DragRowOp->PendingDropAction == FSourceControlHistoryRowDragDropOp::EDropAction::Diff)
			{
				check(DragRowOp->SelectedItems.Num() > 0);
				DiffHistoryItems(DragRowOp->SelectedItems[0], HoveredItem);

				return FReply::Handled();
			}
		}
		return FReply::Unhandled();
	}

	/** Main list view of the panel, displays each file history item */
	typedef STreeView< TSharedPtr<FHistoryTreeItem> > SHistoryFileListType;

	/** ListBox for selecting which object to consolidate */
	TSharedPtr< SHistoryFileListType > MainHistoryListView;

	/** Items control for the "additional information" subpanel */
	TSharedPtr<SBorder> AdditionalInfoItemsControl;

	/** All file history items the panel should display */
	TArray< TSharedPtr<FHistoryTreeItem> > HistoryCollection;

	/** The last selected revision item; Displayed in the "additional information" subpanel */
	TWeakPtr<FHistoryRevisionListViewItem> LastSelectedRevisionItem;

	/** Pointer to the parent window */
	TWeakPtr<SWindow> ParentWindow;
};

void SSourceControlHistoryWidget::CreateDiffMenu(UToolMenu* InToolMenu)
{
	USourceControlHistoryWidgetContext* FoundContext = InToolMenu->FindContext<USourceControlHistoryWidgetContext>();
	if (!FoundContext)
	{
		return;
	}

	TSharedPtr<SSourceControlHistoryWidget> PinnedHistoryWidget = FoundContext->HistoryWidget.Pin();
	if (!PinnedHistoryWidget)
	{
		return;
	}

	if (!CanDiff())
	{
		return;
	}

	FToolMenuSection& DiffSection = InToolMenu->AddSection("DiffTools");
	DiffSection.AddMenuEntry(
		TEXT("DiffAgainstPrevious"),
		NSLOCTEXT("SourceControl.HistoryWindow.Menu", "DiffAgainstPrev", "Diff Against Previous Revision"),
		NSLOCTEXT("SourceControl.HistoryWindow.Menu", "DiffAgainstPrevTooltip", "See changes between this revision and the previous one."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.Actions.Diff"),
		FUIAction(
			FExecuteAction::CreateSP(PinnedHistoryWidget.Get(), &SSourceControlHistoryWidget::OnDiffAgainstPreviousRev),
			FCanExecuteAction::CreateSP(PinnedHistoryWidget.Get(), &SSourceControlHistoryWidget::CanDiffAgainstPreviousRev)
		)
	);

	DiffSection.AddMenuEntry(
		TEXT("DiffAgainstWorkspace"),
		NSLOCTEXT("SourceControl.HistoryWindow.Menu", "DiffAgainstWorkspace", "Diff Against Workspace File"),
		NSLOCTEXT("SourceControl.HistoryWindow.Menu", "DiffAgainstWorkspaceTooltip", "See changes between this revision and your version of the asset."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.Actions.Diff"),
		FUIAction(
			FExecuteAction::CreateSP(PinnedHistoryWidget.Get(), &SSourceControlHistoryWidget::OnDiffAgainstWorkspace),
			FCanExecuteAction::CreateSP(PinnedHistoryWidget.Get(), &SSourceControlHistoryWidget::CanDiffAgainstWorkspace)
		)
	);

	DiffSection.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateSP(PinnedHistoryWidget.Get(), &SSourceControlHistoryWidget::CreateDiffSelectedMenu));
}

void FSourceControlWindows::DisplayRevisionHistory( const TArray<FString>& InPackageNames )
{
	// Explicitly load the module so live coding will work with it
	FModuleManager::Get().LoadModule(TEXT("SourceControlWindows"));

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	// Query for the file history for the provided packages
	TArray<FString> PackageFilenames = SourceControlHelpers::PackageFilenames(InPackageNames);
	TSharedRef<FUpdateStatus, ESPMode::ThreadSafe> UpdateStatusOperation = ISourceControlOperation::Create<FUpdateStatus>();
	UpdateStatusOperation->SetUpdateHistory(true);
	if(SourceControlProvider.Execute(UpdateStatusOperation, PackageFilenames))
	{
		TArray< FSourceControlStateRef > SourceControlStates;
		for(const FString& PackageFilename : PackageFilenames)
		{
			TArray<FString> RevisionName;
			RevisionName.Add(PackageFilename);

			while(RevisionName.Num() != 0)
			{
				int32 InitialNum = SourceControlStates.Num();
				SourceControlProvider.GetState(RevisionName, SourceControlStates, EStateCacheUsage::Use);
				int32 NewNum = SourceControlStates.Num();
				ensure(NewNum >= InitialNum);
				
				RevisionName.Empty();
				// check to see if origin of this file is a branch, append the history from the branch point:
				if(NewNum > InitialNum)
				{
					int32 HistorySize = SourceControlStates.Last()->GetHistorySize();
					if( HistorySize > 0 )
					{
						TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> InitialHistory = SourceControlStates.Last()->GetHistoryItem(HistorySize - 1);
						TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> BranchSource = InitialHistory->GetBranchSource();
						if( BranchSource.IsValid() )
						{
							RevisionName.Add(BranchSource->GetFilename());
						}
					}
				}
			}
		}

		TSharedRef<SWindow> NewWindow = SNew(SWindow)
			.Title( NSLOCTEXT("SourceControl.HistoryWindow", "Title", "File History") )
			.SizingRule(ESizingRule::UserSized)
			.AutoCenter(EAutoCenter::PreferredWorkArea)
			.ClientSize(FVector2D(1000, 400));

		TSharedRef<SSourceControlHistoryWidget> SourceControlWidget = 
			SNew(SSourceControlHistoryWidget)
			.ParentWindow(NewWindow)
			.SourceControlStates(SourceControlStates);


		NewWindow->SetContent( SourceControlWidget );

		TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();
		if(RootWindow.IsValid())
		{
			FSlateApplication::Get().AddWindowAsNativeChild(NewWindow, RootWindow.ToSharedRef());
		}
		else
		{
			FSlateApplication::Get().AddWindow(NewWindow);
		}
	}
}

bool FSourceControlWindows::DiffAgainstWorkspace(const FString& InFileName)
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));

	UObject* SelectedAsset = nullptr;
	FString AssetPackageName;

	if (FPackageName::TryConvertFilenameToLongPackageName(InFileName, AssetPackageName))
	{
		UPackage* AssetPackage = LoadPackage(nullptr, *AssetPackageName, LOAD_None);

		// grab the asset from the package - we assume asset name matches file name
		FString AssetName = FPaths::GetBaseFilename(InFileName);
		SelectedAsset = FindObject<UObject>(AssetPackage, *AssetName);

		if (!SelectedAsset && AssetPackage)
		{
			SelectedAsset = AssetPackage->FindAssetInPackage();
		}
	}

	if (SelectedAsset)
	{
		AssetToolsModule.Get().DiffAgainstDepot(SelectedAsset, AssetPackageName, SelectedAsset->GetName());
	}

	return !!SelectedAsset;
}

bool FSourceControlWindows::DiffAgainstShelvedFile(const FSourceControlStateRef& InFileState)
{
	if (InFileState->GetHistorySize() == 0)
	{
		return false;
	}

	TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> Revision = InFileState->GetHistoryItem(0);
	check(Revision.IsValid());

	UObject* SelectedAsset = nullptr;

	FString AssetPackageName;
	if (FPackageName::TryConvertFilenameToLongPackageName(InFileState->GetFilename(), AssetPackageName))
	{
		UPackage* AssetPackage = LoadPackage(nullptr, *AssetPackageName, LOAD_None);

		// grab the asset from the package - we assume asset name matches file name
		FString AssetName = FPaths::GetBaseFilename(InFileState->GetFilename());
		SelectedAsset = FindObject<UObject>(AssetPackage, *AssetName);

		if (!SelectedAsset && AssetPackage)
		{
			SelectedAsset = AssetPackage->FindAssetInPackage();
		}
	}

	if (SelectedAsset)
	{
		FString TempFileName;
		if (Revision->Get(TempFileName))
		{
			// Try and load that package
			const FPackagePath TempPackagePath = FPackagePath::FromLocalPath(TempFileName);
			const FPackagePath OriginalPackagePath = FPackagePath::FromLocalPath(InFileState->GetFilename());
			if(UPackage* TempPackage = DiffUtils::LoadPackageForDiff(TempPackagePath, OriginalPackagePath))
			{
				// Grab the shelved asset from that package
				UObject* ShelvedObject = FindObject<UObject>(TempPackage, *SelectedAsset->GetName());

				if (ShelvedObject == nullptr)
				{
					ShelvedObject = TempPackage->FindAssetInPackage();
				}

				if (ShelvedObject != nullptr)
				{
					/* Set the revision information*/
					FRevisionInfo ShelvedRevision;
					ShelvedRevision.Changelist = Revision->GetCheckInIdentifier();
					ShelvedRevision.Revision = TEXT("Shelved");
					ShelvedRevision.Date = Revision->GetDate();

					FRevisionInfo NewRevision;
					NewRevision.Revision = TEXT("");

					FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
					AssetToolsModule.Get().DiffAssets(ShelvedObject, SelectedAsset, ShelvedRevision, NewRevision);
				}
			}
		}
	}

	return !!SelectedAsset;
}