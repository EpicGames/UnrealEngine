// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "AssetToolsModule.h"
#include "Misc/MessageDialog.h"
#include "IAssetTools.h"
#include "ISourceControlOperation.h"
#include "RevisionControlStyle/RevisionControlStyle.h"
#include "SourceControlOperations.h"
#include "SourceControlWindows.h"
#include "SourceControlHelpers.h"
#include "SSourceControlChangelistRows.h"
#include "ISourceControlModule.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SWindow.h"
#include "SlateOptMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "PackageTools.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "UObject/Linker.h"
#include "FileHelpers.h"

#define LOCTEXT_NAMESPACE "SSourceControlRevert"

//-------------------------------------
//Source Control Window Constants
//-------------------------------------
enum class ERevertResults
{
	Accepted,
	Canceled
};

/** Additional state for each IFileViewTreeItem row. */
struct FRevertTableRowState
{
	FRevertTableRowState(const FString& InPackageName, bool bInIsModified)
		: bIsModified(bInIsModified)
		, PackageName(InPackageName)
	{
	}

	const bool bIsModified;
	const FString PackageName;
};

/** Row widget - derives from either SFileTableRow or SOfflineFileTableRow depending on whether we are performing a unsaved revert or not. */
template<typename CommonTableRowType>
class SSourceControlRevertRow : public CommonTableRowType
{
public:
	SLATE_BEGIN_ARGS(SSourceControlRevertRow)
		: _ShowingContentVersePath(false)
		{
		}

		SLATE_ARGUMENT(FChangelistTreeItemPtr, TreeItemToVisualize)
		SLATE_ARGUMENT(bool, IsModified)
		SLATE_ARGUMENT(bool, ShowingContentVersePath)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwner)
	{
		bIsModified = InArgs._IsModified;

		CommonTableRowType::Construct(
			typename CommonTableRowType::FArguments()
			.TreeItemToVisualize(InArgs._TreeItemToVisualize)
			.PathFlags(InArgs._ShowingContentVersePath ? SourceControlFileViewColumn::EPathFlags::ShowingVersePath : SourceControlFileViewColumn::EPathFlags::Default),
			InOwner);
	}

	TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		if (ColumnName == SourceControlFileViewColumn::Name::Id() && bIsModified)
		{
			// If the item is modified, wrap the name widget with the modified icon.
			return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.AutoWidth()
				[
					CommonTableRowType::GenerateWidgetForColumn(ColumnName)
				]
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(TEXT("ContentBrowser.ContentDirty")))
					.ToolTipText(LOCTEXT("ModifiedFileToolTip", "This file has been modified from the source version"))
				];
		}

		return CommonTableRowType::GenerateWidgetForColumn(ColumnName);
	}

private:
	bool bIsModified = false;
};

/** Returns whether revert unsaved is enabled */
static bool IsRevertUnsavedEnabled()
{
	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("SourceControl.RevertUnsaved.Enable")))
	{
		return CVar->GetBool();
	}
	else
	{
		return false;
	}
}

/**
 * Source control panel for reverting files. Allows the user to select which files should be reverted, as well as
 * provides the option to only allow unmodified files to be reverted.
 */
class SSourceControlRevertWidget : public SCompoundWidget
{
public:

	//* @param	InXamlName		Name of the XAML file defining this panel
	//* @param	InPackageNames	Names of the packages to be potentially reverted
	SLATE_BEGIN_ARGS( SSourceControlRevertWidget )
		: _ParentWindow()
		, _PackagesToRevert()
	{}

		SLATE_ATTRIBUTE( TSharedPtr<SWindow>, ParentWindow )	
		SLATE_ATTRIBUTE( TArray<FString>, PackagesToRevert )

	SLATE_END_ARGS()

	/**
	 * Constructor.
	 */
	SSourceControlRevertWidget()
	{
	}

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	void Construct( const FArguments& InArgs )
	{
		ParentFrame = InArgs._ParentWindow.Get();

		bShowingContentVersePath = FAssetToolsModule::GetModule().Get().ShowingContentVersePath();

		InitializeListViewItemSource(InArgs._PackagesToRevert.Get());
		SortListViewItemSource();

		TSharedRef<SHeaderRow> HeaderRowWidget = SNew(SHeaderRow);

		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			.Padding(FMargin(16))
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 0.0f, 0.0f, 8.0f)
				[
					SNew(STextBlock)
					.Text(NSLOCTEXT("SourceControl.Revert", "SelectFiles", "Select the files that should be reverted below"))
				]

				+SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					SAssignNew(RevertListView, SListViewType)
					.HeaderRow
					(
						SNew(SHeaderRow)
						+SHeaderRow::Column(SourceControlFileViewColumn::CheckBox::Id())
						.FixedWidth(38.0f)
						[
							SNew(SBox)
							.Padding(FMargin(6.0f, 3.0f, 6.0f, 3.0f))
							.HAlign(HAlign_Center)
							[
								SNew(SCheckBox)
								.IsChecked(this, &SSourceControlRevertWidget::OnGetColumnHeaderState)
								.IsEnabled(this, &SSourceControlRevertWidget::OnGetItemsEnabled)
								.OnCheckStateChanged(this, &SSourceControlRevertWidget::ColumnHeaderClicked)
							]
						]

						+ SHeaderRow::Column(SourceControlFileViewColumn::Icon::Id())
						.DefaultTooltip(SourceControlFileViewColumn::Icon::GetToolTipText())
						.FillSized(18.0f)
						.HeaderContentPadding(FMargin(0.0f))
						.SortMode(this, &SSourceControlRevertWidget::GetColumnSortMode, SourceControlFileViewColumn::Icon::Id())
						.OnSort(this, &SSourceControlRevertWidget::OnColumnSortModeChanged)
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.Padding(1.0f, 0.0f)
							[
								SNew(SBox)
								.WidthOverride(16.0f)
								.HeightOverride(16.0f)
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								.Visibility(this, &SSourceControlRevertWidget::GetIconColumnContentVisibility)
								[
									SNew(SImage)
									.ColorAndOpacity(FSlateColor::UseSubduedForeground())
									.Image(FRevisionControlStyleManager::Get().GetBrush("RevisionControl.Icon"))
								]
							]
						]

						+SHeaderRow::Column(SourceControlFileViewColumn::Name::Id())
						.DefaultLabel(LOCTEXT("Asset", "Asset"))
						.DefaultTooltip(SourceControlFileViewColumn::Name::GetToolTipText())
						.FillWidth(5.0f)
						.SortMode(this, &SSourceControlRevertWidget::GetColumnSortMode, SourceControlFileViewColumn::Name::Id())
						.OnSort(this, &SSourceControlRevertWidget::OnColumnSortModeChanged)

						+SHeaderRow::Column(SourceControlFileViewColumn::Path::Id())
						.DefaultLabel(LOCTEXT("File", "File"))
						.DefaultTooltip(SourceControlFileViewColumn::Path::GetToolTipText())
						.FillWidth(7.0f)
						.SortMode(this, &SSourceControlRevertWidget::GetColumnSortMode, SourceControlFileViewColumn::Path::Id())
						.OnSort(this, &SSourceControlRevertWidget::OnColumnSortModeChanged)

						+SHeaderRow::Column(SourceControlFileViewColumn::Type::Id())
						.DefaultLabel(SourceControlFileViewColumn::Type::GetDisplayText())
						.DefaultTooltip(SourceControlFileViewColumn::Type::GetToolTipText())
						.FillWidth(2.0f)
						.SortMode(this, &SSourceControlRevertWidget::GetColumnSortMode, SourceControlFileViewColumn::Type::Id())
						.OnSort(this, &SSourceControlRevertWidget::OnColumnSortModeChanged)
					)
					.ListItemsSource(&ListViewItemSource)
					.SelectionMode(ESelectionMode::None)
					.OnGenerateRow(this, &SSourceControlRevertWidget::OnGenerateRowForList)
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0.0f, 16.0f, 0.0f, 0.0f)
				.VAlign(VAlign_Bottom)
				.HAlign(HAlign_Fill)
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(5.0f, 0.0f)
					.HAlign(HAlign_Left)
					[
						SNew(SCheckBox)
						.OnCheckStateChanged(this, &SSourceControlRevertWidget::RevertUnchangedToggled)
						[
							SNew(STextBlock)
							.Text(NSLOCTEXT("SourceControl.Revert", "RevertUnchanged", "Revert Unchanged Only"))
						]
					]

					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.FillWidth(1)
					[
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(5.0f, 0.0f)
						[
							SNew(SButton) 
							.ButtonStyle(&FAppStyle::Get(), "PrimaryButton")
							.TextStyle(&FAppStyle::Get(), "PrimaryButtonText")
							.HAlign(HAlign_Center)
							.OnClicked(this, &SSourceControlRevertWidget::OKClicked)
							.IsEnabled(this, &SSourceControlRevertWidget::IsOKEnabled)
							.Text(this, &SSourceControlRevertWidget::GetOkText)
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(5.0f, 0.0f)
						[
							SNew(SButton) 
							.HAlign(HAlign_Center)
							.OnClicked(this, &SSourceControlRevertWidget::CancelClicked)
							.Text(LOCTEXT("CancelButton", "Cancel"))
						]
					]
				]
			]
		];
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION

	/**
	 * Populates the provided array with the names of the packages the user elected to revert, if any.
	 *
	 * @param	OutPackagesToRevert	Array of package names to revert, as specified by the user in the dialog
	 */
	void GetPackagesToRevert( TArray<FString>& OutPackagesToRevert )
	{
		if (bRevertUnchangedFilesOnly)
		{
			for (const TPair<FChangelistTreeItemPtr, TSharedRef<FRevertTableRowState>>& Pair : ListViewItemState)
			{
				if (!Pair.Value->bIsModified)
				{
					OutPackagesToRevert.Add(Pair.Value->PackageName);
				}
			}
		}
		else
		{
			for (const FChangelistTreeItemPtr& Item : ListViewItemSource)
			{
				if (static_cast<IFileViewTreeItem*>(Item.Get())->GetCheckBoxState() == ECheckBoxState::Checked)
				{
					const TSharedRef<FRevertTableRowState>* State = ListViewItemState.Find(Item);
					if (ensure(State))
					{
						OutPackagesToRevert.Add((*State)->PackageName);
					}
				}
			}
		}
	}


	ERevertResults GetResult()
	{
		return DialogResult;
	}

private:
	EColumnSortMode::Type GetColumnSortMode(FName ColumnId) const
	{
		if (SortByColumn != ColumnId)
		{
			return EColumnSortMode::None;
		}

		return SortMode;
	}

	void OnColumnSortModeChanged(EColumnSortPriority::Type SortPriority, const FName& ColumnId, EColumnSortMode::Type InSortMode)
	{
		SortByColumn = ColumnId;
		SortMode = InSortMode;

		SortListViewItemSource();
		RevertListView->RequestListRefresh();
	}

	EVisibility GetIconColumnContentVisibility() const
	{
		// Hide the icon when sorting the icon column (it clashes with the sort mode icon).
		return GetColumnSortMode(SourceControlFileViewColumn::Icon::Id()) == EColumnSortMode::None ? EVisibility::Visible : EVisibility::Collapsed;
	}

	TSharedRef<ITableRow> OnGenerateRowForList(FChangelistTreeItemPtr ListItemPtr, const TSharedRef<STableViewBase>& OwnerTable)
	{
		TSharedRef<FRevertTableRowState> State = ListViewItemState.FindChecked(ListItemPtr);

		switch (ListItemPtr->GetTreeItemType())
		{
		case IChangelistTreeItem::File:
			return SNew(SSourceControlRevertRow<SFileTableRow>, OwnerTable)
				.IsEnabled(this, &SSourceControlRevertWidget::OnGetItemsEnabled)
				.TreeItemToVisualize(ListItemPtr)
				.IsModified(State->bIsModified)
				.ShowingContentVersePath(bShowingContentVersePath);
		case IChangelistTreeItem::OfflineFile:
			return SNew(SSourceControlRevertRow<SOfflineFileTableRow>, OwnerTable)
				.IsEnabled(this, &SSourceControlRevertWidget::OnGetItemsEnabled)
				.TreeItemToVisualize(ListItemPtr)
				.IsModified(State->bIsModified)
				.ShowingContentVersePath(bShowingContentVersePath);
		default:
			checkNoEntry();
			return SNew(STableRow<TSharedPtr<FChangelistTreeItemPtr>>, OwnerTable);
		}
	}

	/** Called when the settings of the dialog are to be accepted*/
	FReply OKClicked()
	{
		DialogResult = ERevertResults::Accepted;
		ParentFrame.Pin()->RequestDestroyWindow();

		return FReply::Handled();
	}

	bool IsOKEnabled() const
	{
		if (bRevertUnchangedFilesOnly)
		{
			for (const TPair<FChangelistTreeItemPtr, TSharedRef<FRevertTableRowState>>& Pair : ListViewItemState)
			{
				if (!Pair.Value->bIsModified)
				{
					return true;
				}
			}
		}
		else
		{
			for (const FChangelistTreeItemPtr& Item : ListViewItemSource)
			{
				if (static_cast<IFileViewTreeItem*>(Item.Get())->GetCheckBoxState() == ECheckBoxState::Checked)
				{
					return true;
				}
			}
		}

		return false;
	}

	FText GetOkText() const
	{
		if (bRevertUnchangedFilesOnly)
		{
			return LOCTEXT("RevertUnchangedButton", "Revert Unchanged");
		}

		return LOCTEXT("RevertButton", "Revert Selected");
	}

	/** Called when the settings of the dialog are to be ignored*/
	FReply CancelClicked()
	{
		DialogResult = ERevertResults::Canceled;
		ParentFrame.Pin()->RequestDestroyWindow();

		return FReply::Handled();
	}

	/** Called when the user checks or unchecks the revert unchanged checkbox; updates the list view accordingly */
	void RevertUnchangedToggled( const ECheckBoxState NewCheckedState )
	{
		bRevertUnchangedFilesOnly = (NewCheckedState == ECheckBoxState::Checked);
	}

	ECheckBoxState OnGetColumnHeaderState() const
	{
		int32 NumChecked = 0;
		for (const FChangelistTreeItemPtr& Item : ListViewItemSource)
		{
			switch (static_cast<IFileViewTreeItem*>(Item.Get())->GetCheckBoxState())
			{
			case ECheckBoxState::Checked:
				++NumChecked;
				break;
			case ECheckBoxState::Undetermined:
				return ECheckBoxState::Undetermined;
			}
		}

		if (NumChecked == 0)
		{
			return ECheckBoxState::Unchecked;
		}

		if (NumChecked == ListViewItemSource.Num())
		{
			return ECheckBoxState::Checked;
		}

		return ECheckBoxState::Undetermined;
	}

	/**
	 * Called whenever a column header is clicked, or in the case of the dialog, also when the "Check/Uncheck All" column header
	 * checkbox is called, because its event bubbles to the column header. 
	 */
	void ColumnHeaderClicked( const ECheckBoxState NewCheckedState )
	{
		for (const FChangelistTreeItemPtr& Item : ListViewItemSource)
		{
			static_cast<IFileViewTreeItem*>(Item.Get())->SetCheckBoxState(NewCheckedState);
		}
	}

	/** Initializes the current state of the files, */
	void InitializeListViewItemSource(const TArray<FString>& PackagesToRevert)
	{
		const bool bRevertUnsaved = IsRevertUnsavedEnabled();

		TArray<FString> PackageFilenames = SourceControlHelpers::PackageFilenames(PackagesToRevert);

		// Make sure we update the modified state of the files
		TSharedRef<FUpdateStatus, ESPMode::ThreadSafe> UpdateStatusOperation = ISourceControlOperation::Create<FUpdateStatus>();
		UpdateStatusOperation->SetUpdateModifiedState(true);
		ISourceControlModule::Get().GetProvider().Execute(UpdateStatusOperation, PackageFilenames);

		// Find the files modified from the server version
		TArray<FSourceControlStateRef> SourceControlStates;
		if (ISourceControlModule::Get().GetProvider().GetState(PackageFilenames, SourceControlStates, EStateCacheUsage::Use))
		{
			ListViewItemSource.Reserve(SourceControlStates.Num());
			for (const FSourceControlStateRef& SourceControlState : SourceControlStates)
			{
				FString PackageName;
				if (ensure(FPackageName::TryConvertFilenameToLongPackageName(SourceControlState->GetFilename(), PackageName)))
				{
					const FChangelistTreeItemPtr& ListViewItem = ListViewItemSource.Add_GetRef(MakeShared<FFileTreeItem>(SourceControlState));

					bool bIsModified = SourceControlState->IsModified();
					if (bRevertUnsaved && !bIsModified)
					{
						if (UPackage* Package = FindPackage(nullptr, *PackageName))
						{
							// If the package contains unsaved changes, it's considered modified as well.
							bIsModified = Package->IsDirty();
						}
					}

					ListViewItemState.Emplace(ListViewItem, MakeShared<FRevertTableRowState>(PackageName, bIsModified));
				}
			}
		}
		else if (bRevertUnsaved)
		{
			ListViewItemSource.Reserve(PackagesToRevert.Num());
			for (int32 Index = 0; Index < PackagesToRevert.Num(); ++Index)
			{
				const FChangelistTreeItemPtr& ListViewItem = ListViewItemSource.Add_GetRef(MakeShared<FOfflineFileTreeItem>(PackageFilenames[Index]));

				bool bIsModified = false;
				if (UPackage* Package = FindPackage(nullptr, *PackagesToRevert[Index]))
				{
					// If the package contains unsaved changes, it's considered modified.
					bIsModified = Package->IsDirty();
				}

				ListViewItemState.Emplace(ListViewItem, MakeShared<FRevertTableRowState>(PackagesToRevert[Index], bIsModified));
			}
		}
	}

	void SortListViewItemSource()
	{
		TFunction<bool(const IFileViewTreeItem&, const IFileViewTreeItem&)> SortPredicate = SourceControlFileViewColumn::GetSortPredicate(
			SortMode, SortByColumn, bShowingContentVersePath ? SourceControlFileViewColumn::EPathFlags::ShowingVersePath : SourceControlFileViewColumn::EPathFlags::Default);
		if (SortPredicate)
		{
			Algo::SortBy(
				ListViewItemSource,
				[](const FChangelistTreeItemPtr& ListViewItem) -> const IFileViewTreeItem&
				{
					return static_cast<IFileViewTreeItem&>(*ListViewItem);
				},
				SortPredicate);
		}
	}

	/** Check for whether the list items are enabled or not */
	bool OnGetItemsEnabled() const
	{
		return !bRevertUnchangedFilesOnly;
	}

	TWeakPtr<SWindow> ParentFrame;
	ERevertResults DialogResult = ERevertResults::Canceled;

	/** ListView for the packages the user can revert */
	typedef SListView<FChangelistTreeItemPtr> SListViewType;
	TSharedPtr<SListViewType> RevertListView;

	FName SortByColumn = SourceControlFileViewColumn::Name::Id();
	EColumnSortMode::Type SortMode = EColumnSortMode::Ascending;

	/** Collection of items serving as the data source for the list view */
	TArray<FChangelistTreeItemPtr> ListViewItemSource;

	TMap<FChangelistTreeItemPtr, TSharedRef<FRevertTableRowState>> ListViewItemState;

	bool bShowingContentVersePath = false;

	/** Flag set by the user to only revert non modified files */
	bool bRevertUnchangedFilesOnly = false;
};

bool FSourceControlWindows::PromptForRevert( const TArray<FString>& InPackageNames, bool bInReloadWorld)
{
	bool bReverted = false;

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	// Only add packages that can actually be reverted
	TArray<FString> InitialPackagesToRevert;
	for ( TArray<FString>::TConstIterator PackageIter( InPackageNames ); PackageIter; ++PackageIter )
	{
		FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(SourceControlHelpers::PackageFilename(*PackageIter), EStateCacheUsage::Use);
		if( SourceControlState.IsValid() && SourceControlState->CanRevert() )
		{
			InitialPackagesToRevert.Add( *PackageIter );
		}
		else if ( IsRevertUnsavedEnabled() )
		{
			if (UPackage* Package = FindPackage(NULL, **PackageIter))
			{
				if (Package->IsDirty())
				{
					InitialPackagesToRevert.Add(*PackageIter);
				}
			}
		}
	}

	// If any of the packages can be reverted, provide the revert prompt
	if (InitialPackagesToRevert.Num() > 0)
	{
		TSharedRef<SWindow> NewWindow = SNew(SWindow)
			.Title( NSLOCTEXT("SourceControl.RevertWindow", "Title", "Revert Files") )
			.ClientSize(FVector2D(640.0f, 492.0f))
			.SupportsMinimize(false) 
			.SupportsMaximize(false);

		TSharedRef<SSourceControlRevertWidget> SourceControlWidget = 
			SNew(SSourceControlRevertWidget)
			.ParentWindow(NewWindow)
			.PackagesToRevert(InitialPackagesToRevert);

		NewWindow->SetContent(SourceControlWidget);

		FSlateApplication::Get().AddModalWindow(NewWindow, NULL);

		// If the user decided to revert some packages, go ahead and do revert the ones they selected
		if ( SourceControlWidget->GetResult() == ERevertResults::Accepted)
		{
			TArray<FString> FinalPackagesToRevert;
			SourceControlWidget->GetPackagesToRevert(FinalPackagesToRevert);
			
			if ( IsRevertUnsavedEnabled() )
			{
				// Unsaved changes need to be saved to disk so SourceControl realizes that there's something to revert.

				TArray<UPackage*> FinalPackagesToSave;
				for (const FString& PackageName : FinalPackagesToRevert)
				{
					if (UPackage* Package = FindPackage(NULL, *PackageName))
					{
						if (Package->IsDirty())
						{
							FinalPackagesToSave.Add(Package);
						}
					}
				}

				if (FinalPackagesToSave.Num() > 0)
				{
					UEditorLoadingAndSavingUtils::SavePackages(FinalPackagesToSave, /*bOnlyDirty=*/false);
				}
			}

			if (FinalPackagesToRevert.Num() > 0)
			{
				SourceControlHelpers::RevertAndReloadPackages(FinalPackagesToRevert, /*bRevertAll=*/false, /*bReloadWorld=*/bInReloadWorld);

				bReverted = true;
			}
		}
	}

	return bReverted;
}

bool FSourceControlWindows::RevertAllChangesAndReloadWorld()
{	
	return SourceControlHelpers::RevertAllChangesAndReloadWorld();
}

#undef LOCTEXT_NAMESPACE
