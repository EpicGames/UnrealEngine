// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSourceControlSubmit.h"

#include "IAssetTools.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "SourceControlHelpers.h"
#include "SourceControlWindows.h"
#include "SSourceControlCommon.h"
#include "SSourceControlChangelistRows.h"
#include "Modules/ModuleManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/Input/SCheckBox.h"
#include "UObject/UObjectHash.h"
#include "Styling/AppStyle.h"
#include "AssetToolsModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Virtualization/VirtualizationSystem.h"
#include "Logging/MessageLog.h"
#include "RevisionControlStyle/RevisionControlStyle.h"
#include "HAL/IConsoleManager.h"
#include "Algo/AllOf.h"
#include "ToolMenu.h"
#include "ToolMenus.h"

#if SOURCE_CONTROL_WITH_SLATE

#define LOCTEXT_NAMESPACE "SSourceControlSubmit"

// This is useful for source control that do not support changelist (Git/SVN) or when the submit widget is not created from the changelist window. If a user
// commits/submits this way, then edits the submit description but cancels, the description will be remembered in memory for the next time he tries to submit.
static FText GSavedChangeListDescription;

bool TryToVirtualizeFilesToSubmit(const TArray<FString>& FilesToSubmit, FText& Description, FText& OutFailureMsg)
{
	using namespace UE::Virtualization;

	IVirtualizationSystem& System = IVirtualizationSystem::Get();
	if (!System.IsEnabled())
	{
		return true; // Early out if VA is not enabled
	}

	TArray<FSourceControlStateRef> FileStates;

	if (ISourceControlModule::Get().GetProvider().GetState(FilesToSubmit, FileStates, EStateCacheUsage::Use) == ECommandResult::Succeeded)
	{
		return TryToVirtualizeFilesToSubmit(FileStates, Description, OutFailureMsg);
	}
	else
	{
		OutFailureMsg = LOCTEXT("SCC_VA_GetStateFailed", "Failed to resolve the file states from revision control!");
		return false;
	}
}

bool TryToVirtualizeFilesToSubmit(const TArray<FSourceControlStateRef>& FileStates, FText& Description, FText& OutFailureMsg)
{
	using namespace UE::Virtualization;

	IVirtualizationSystem& System = IVirtualizationSystem::Get();
	if (!System.IsEnabled())
	{
		return true; // Early out if VA is not enabled
	}

	TArray<FString> FilesToSubmit;
	{
		FilesToSubmit.Reserve(FileStates.Num());
		for (const FSourceControlStateRef& State : FileStates)
		{
			if (State->IsDeleted())
			{
				UE_LOGF(LogVirtualization, Verbose, "Ignoring package marked for delete '%ls'", *State->GetFilename());
				continue;
			}

			if (State->IsIgnored())
			{
				UE_LOGF(LogVirtualization, Verbose, "Ignoring package marked for ignore '%ls'", *State->GetFilename());
				continue;
			}

			FilesToSubmit.Add(State->GetFilename());
		}
	}

	{
		TArray<FText> PayloadErrors;
		TArray<FText> DescriptionTags;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ISourceControlModule::Get().GetOnPreSubmitFinalize().Broadcast(FilesToSubmit, DescriptionTags, PayloadErrors);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	EVirtualizationOptions VirtualizationOptions = EVirtualizationOptions::None;

	FVirtualizationResult Result = System.TryVirtualizePackages(FilesToSubmit, VirtualizationOptions);
	if (Result.WasSuccessful())
	{
		FTextBuilder NewDescription;
		NewDescription.AppendLine(Description);

		for (const FText& Line : Result.DescriptionTags)
		{
			NewDescription.AppendLine(Line);
		}

		Description = NewDescription.ToText();

		return true;
	}
	else if (System.AllowSubmitIfVirtualizationFailed())
	{
		for (const FText& Error : Result.Errors)
		{
			FMessageLog("SourceControl").Warning(Error);
		}

		// Even though the virtualization process had problems we should continue submitting
		return true;
	}
	else
	{
		for (const FText& Error : Result.Errors)
		{
			FMessageLog("SourceControl").Error(Error);
		}

		OutFailureMsg = LOCTEXT("SCC_Virtualization_Failed", "Failed to virtualize the files being submitted!");

		return false;
	}
}


SSourceControlSubmitWidget::~SSourceControlSubmitWidget()
{
	// If the user cancel the submit, save the changelist. If the user submitted, ChangeListDescriptionTextCtrl was cleared).
	GSavedChangeListDescription = ChangeListDescriptionTextCtrl->GetText();
}

void SSourceControlSubmitWidget::Construct(const FArguments& InArgs)
{
	ParentFrame = InArgs._ParentWindow.Get();
	SortByColumn = SourceControlFileViewColumn::Name::Id();
	SortMode = EColumnSortMode::Ascending;
	if (!InArgs._Description.Get().IsEmpty())
	{
		// If a description is provided, override the last one saved in memory.
		GSavedChangeListDescription = InArgs._Description.Get();
	}
	bAllowSubmit = InArgs._AllowSubmit.Get();
	bAllowDiffAgainstDepot = InArgs._AllowDiffAgainstDepot.Get();
	// This widget is only used in a modal window, so bShowingContentVersePath shouldn't change.
	bShowingContentVersePath = FAssetToolsModule::GetModule().Get().ShowingContentVersePath();

	const bool bDescriptionIsReadOnly = !InArgs._AllowDescriptionChange.Get();
	const bool bAllowUncheckFiles = InArgs._AllowUncheckFiles.Get();
	const bool bAllowKeepCheckedOut = InArgs._AllowKeepCheckedOut.Get();
	const bool bShowChangelistValidation = !InArgs._ChangeValidationResult.Get().IsEmpty();
	const bool bAllowSaveAndClose = InArgs._AllowSaveAndClose.Get();

	for (const auto& Item : InArgs._Items.Get())
	{
		ListViewItems.Add(MakeShared<FFileTreeItem>(Item));
	}

	if (CanCheckOut() && !bAllowUncheckFiles)
	{
		for (const auto& Item : InArgs._Items.Get())
		{
			if (Item->IsCheckedOutOther() && !Item->CanCheckout()) // Is it an exclusive checkout?
			{
				CheckedOutByOther.Add(Item);
			}
		}
	}

	TSharedRef<SHeaderRow> HeaderRowWidget = SNew(SHeaderRow);

	if (bAllowUncheckFiles)
	{
		HeaderRowWidget->AddColumn(
			SHeaderRow::Column(SourceControlFileViewColumn::CheckBox::Id())
			[
				SNew(SBox)
				.Padding(FMargin(6.0f, 3.0f, 6.0f, 3.0f))
				.HAlign(HAlign_Center)
				[
					SNew(SCheckBox)
					.IsChecked(this, &SSourceControlSubmitWidget::GetToggleSelectedState)
					.OnCheckStateChanged(this, &SSourceControlSubmitWidget::OnToggleSelectedCheckBox)
				]
			]
			.FixedWidth(38.0f)
		);
	}

	HeaderRowWidget->AddColumn(
		SHeaderRow::Column(SourceControlFileViewColumn::Icon::Id())
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(1.0f, 0.0f)
			[
				SNew(SBox)
				.WidthOverride(16.0f)
				.HeightOverride(16.0f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Visibility(this, &SSourceControlSubmitWidget::GetIconColumnContentVisibility)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.Image(FRevisionControlStyleManager::Get().GetBrush("RevisionControl.Icon"))
				]
			]
		]
		.DefaultTooltip(SourceControlFileViewColumn::Icon::GetToolTipText())
		.SortMode(this, &SSourceControlSubmitWidget::GetColumnSortMode, SourceControlFileViewColumn::Icon::Id())
		.OnSort(this, &SSourceControlSubmitWidget::OnColumnSortModeChanged)
		.FillSized(18.0f)
		.HeaderContentPadding(FMargin(0.0f))
	);

	HeaderRowWidget->AddColumn(
		SHeaderRow::Column(SourceControlFileViewColumn::Name::Id())
		.DefaultLabel(LOCTEXT("AssetColumnLabel", "Asset"))
		.DefaultTooltip(SourceControlFileViewColumn::Name::GetToolTipText())
		.SortMode(this, &SSourceControlSubmitWidget::GetColumnSortMode, SourceControlFileViewColumn::Name::Id())
		.OnSort(this, &SSourceControlSubmitWidget::OnColumnSortModeChanged)
		.FillWidth(5.0f)
	);

	HeaderRowWidget->AddColumn(
		SHeaderRow::Column(SourceControlFileViewColumn::Path::Id())
		.DefaultLabel(LOCTEXT("FileColumnLabel", "File"))
		.DefaultTooltip(SourceControlFileViewColumn::Path::GetToolTipText())
		.SortMode(this, &SSourceControlSubmitWidget::GetColumnSortMode, SourceControlFileViewColumn::Path::Id())
		.OnSort(this, &SSourceControlSubmitWidget::OnColumnSortModeChanged)
		.FillWidth(7.0f)
	);

	HeaderRowWidget->AddColumn(
		SHeaderRow::Column(SourceControlFileViewColumn::Type::Id())
		.DefaultLabel(SourceControlFileViewColumn::Type::GetDisplayText())
		.DefaultTooltip(SourceControlFileViewColumn::Type::GetToolTipText())
		.SortMode(this, &SSourceControlSubmitWidget::GetColumnSortMode, SourceControlFileViewColumn::Type::Id())
		.OnSort(this, &SSourceControlSubmitWidget::OnColumnSortModeChanged)
		.FillWidth(1.0f)
	);

	TSharedPtr<SVerticalBox> Contents;

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SAssignNew(Contents, SVerticalBox)
		]
	];

	// Build contents of dialog
	Contents->AddSlot()
	.AutoHeight()
	.Padding(5)
	[
		SNew(STextBlock)
		.Text(NSLOCTEXT("SourceControl.SubmitPanel", "ChangeListDesc", "Description"))
	];

	Contents->AddSlot()
	.FillHeight(0.4f)
	.Padding(FMargin(5, 0, 5, 5))
	[
		SAssignNew(ChangeListDescriptionTextCtrl, SMultiLineEditableTextBox)
		.SelectAllTextWhenFocused(!bDescriptionIsReadOnly)
		.Text(GSavedChangeListDescription)
		.AutoWrapText(true)
		.IsReadOnly(bDescriptionIsReadOnly)
	];

	Contents->AddSlot()
	.Padding(FMargin(5, 0))
	.FillHeight(1.f)
	[
		SNew(SBorder)
		[
			SAssignNew(ListView, SListView<FChangelistTreeItemPtr>)
			.ListItemsSource(&ListViewItems)
			.OnGenerateRow(this, &SSourceControlSubmitWidget::OnGenerateRowForList)
			.OnContextMenuOpening(this, &SSourceControlSubmitWidget::OnCreateContextMenu)
			.OnMouseButtonDoubleClick(this, &SSourceControlSubmitWidget::OnDiffAgainstDepotSelected)
			.HeaderRow(HeaderRowWidget)
			.SelectionMode(ESelectionMode::Multi)
		]
	];

	if (!bDescriptionIsReadOnly)
	{
		Contents->AddSlot()
		.AutoHeight()
		.Padding(FMargin(5, 5, 5, 0))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.Visibility_Lambda(
					[this]()
					{
						if (IsWarningPanelVisible() == EVisibility::Visible)
						{
							return IsDescriptionWarningVisible() ? EVisibility::Visible : EVisibility::Collapsed;
						}
						else
						{
							return EVisibility::Collapsed;
						}
					}
				)
				.Padding(5)
				[
					SNew(SErrorText)
					.ErrorText(NSLOCTEXT("SourceControl.SubmitPanel", "ChangeListDescWarning", "Description is required to submit"))
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.Visibility_Lambda(
					[this]()
					{
						if (IsWarningPanelVisible() == EVisibility::Visible)
						{
							return IsValidationWarningVisible() ? EVisibility::Visible : EVisibility::Collapsed;
						}
						else
						{
							return EVisibility::Collapsed;
						}
					}
				)
				.Padding(5)
				[
					SNew(SErrorText)
					.ErrorText(NSLOCTEXT("SourceControl.SubmitPanel", "ChangeListErrorWarning", "Errors need to be fixed to submit"))
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBorder)
				.Visibility_Lambda(
					[this]()
					{
						if (IsWarningPanelVisible() == EVisibility::Visible)
						{
							return IsCheckoutWarningVisible() ? EVisibility::Visible : EVisibility::Collapsed;
						}
						else
						{
							return EVisibility::Collapsed;
						}
					}
				)
				.Padding(5)
				[
					SNew(SErrorText)
					.ErrorText(NSLOCTEXT("SourceControl.SubmitPanel", "ChangeListCheckedOutWarning", "Checkouts by other users need to be cleared to submit"))
				]
			]
		];
	}

	if (bShowChangelistValidation)
	{
		const FString ChangelistResultText = InArgs._ChangeValidationResult.Get();
		const FString ChangelistResultWarningsText = InArgs._ChangeValidationWarnings.Get();
		const FString ChangelistResultErrorsText = InArgs._ChangeValidationErrors.Get();

		const FName ChangelistSuccessIconName = TEXT("Icons.SuccessWithColor.Large");
		const FName ChangelistWarningsIconName = TEXT("Icons.WarningWithColor.Large");
		const FName ChangelistErrorsIconName = TEXT("Icons.ErrorWithColor.Large");

		if (ChangelistResultWarningsText.IsEmpty() && ChangelistResultErrorsText.IsEmpty())
		{
			Contents->AddSlot()
			.AutoHeight()
			.Padding(FMargin(5))
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(ChangelistSuccessIconName))
				]
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				[
					SNew(SMultiLineEditableTextBox)
					.Text(FText::FromString(ChangelistResultText))
					.AutoWrapText(true)
					.IsReadOnly(true)
				]
			];
		}
		else
		{
			Contents->AddSlot()
			.AutoHeight()
			.Padding(FMargin(5))
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				[
					SNew(SMultiLineEditableTextBox)
					.Text(FText::FromString(ChangelistResultText))
					.AutoWrapText(true)
					.IsReadOnly(true)
				]
			];

			if (!ChangelistResultErrorsText.IsEmpty())
			{
				Contents->AddSlot()
				.FillHeight(1.f)
				.Padding(FMargin(5))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush(ChangelistErrorsIconName))
					]
					+SHorizontalBox::Slot()
					[
						SNew(SMultiLineEditableTextBox)
						.Text(FText::FromString(ChangelistResultErrorsText))
						.AutoWrapText(true)
						.IsReadOnly(true)
					]
				];
			}

			if (!ChangelistResultWarningsText.IsEmpty())
			{
				Contents->AddSlot()
				.FillHeight(1.f)
				.Padding(FMargin(5))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush(ChangelistWarningsIconName))
					]
					+SHorizontalBox::Slot()
					[
						SNew(SMultiLineEditableTextBox)
						.Text(FText::FromString(ChangelistResultWarningsText))
						.AutoWrapText(true)
						.IsReadOnly(true)
					]
				];
			}
		}
	}

	if (bAllowKeepCheckedOut)
	{
		Contents->AddSlot()
		.AutoHeight()
		.Padding(5)
		[
			SNew(SWrapBox)
			.UseAllottedSize(true)
			+SWrapBox::Slot()
			.Padding(0.0f, 0.0f, 16.0f, 0.0f)
			[
				SNew(SCheckBox)
				.OnCheckStateChanged( this, &SSourceControlSubmitWidget::OnCheckStateChanged_KeepCheckedOut)
				.IsChecked( this, &SSourceControlSubmitWidget::GetKeepCheckedOut )
				.IsEnabled( this, &SSourceControlSubmitWidget::CanCheckOut )
				[
					SNew(STextBlock)
					.Text(NSLOCTEXT("SourceControl.SubmitPanel", "KeepCheckedOut", "Keep Files Checked Out") )
				]
			]
		];
	}

	const float AdditionalTopPadding = (bAllowKeepCheckedOut ? 0.0f : 5.0f);

	TSharedPtr<SUniformGridPanel> SubmitSaveCancelButtonGrid;
	int32 ButtonSlotId = 0;

	Contents->AddSlot()
	.AutoHeight()
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Bottom)
	.Padding(0.0f, AdditionalTopPadding, 0.0f, 5.0f)
	[
		SAssignNew(SubmitSaveCancelButtonGrid, SUniformGridPanel)
		.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
		.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
		.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
		+SUniformGridPanel::Slot(ButtonSlotId++, 0)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
			.ButtonStyle(&FAppStyle::GetWidgetStyle<FButtonStyle>("PrimaryButton"))
			.IsEnabled(this, &SSourceControlSubmitWidget::IsSubmitEnabled)
			.Text( NSLOCTEXT("SourceControl.SubmitPanel", "OKButton", "Submit") )
			.OnClicked(this, &SSourceControlSubmitWidget::SubmitClicked)
		]
	];

	if (bAllowSaveAndClose)
	{
		SubmitSaveCancelButtonGrid->AddSlot(ButtonSlotId++, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(NSLOCTEXT("SourceControl.SubmitPanel", "Save", "Save"))
				.ToolTipText(NSLOCTEXT("SourceControl.SubmitPanel", "Save_Tooltip", "Save the description and close without submitting."))
				.OnClicked(this, &SSourceControlSubmitWidget::SaveAndCloseClicked)
			];
	}

	SubmitSaveCancelButtonGrid->AddSlot(ButtonSlotId++, 0)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
			.Text( NSLOCTEXT("SourceControl.SubmitPanel", "CancelButton", "Cancel") )
			.OnClicked(this, &SSourceControlSubmitWidget::CancelClicked)
		];

	RequestSort();

	DialogResult = ESubmitResults::SUBMIT_CANCELED;
	KeepCheckedOut = ECheckBoxState::Unchecked;

	ParentFrame.Pin()->SetWidgetToFocusOnActivate(ChangeListDescriptionTextCtrl);
}

/** Corvus: Called to create a context menu when right-clicking on an item */
TSharedPtr<SWidget> SSourceControlSubmitWidget::OnCreateContextMenu()
{
	static const FName MenuName = "SourceControl.SubmitContextMenu";

	// Register menu
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus->IsMenuRegistered(MenuName))
	{
		UToolMenu* RegisteredMenu = ToolMenus->RegisterMenu(MenuName);
		// Add section so it can be used as insert position for menu extensions
		RegisteredMenu->AddSection("Source Control");
	}

	// Create context
	USourceControlSubmitWidgetContext* SourceControlSubmitWidgetContext = NewObject<USourceControlSubmitWidgetContext>();
	SourceControlSubmitWidgetContext->SubmitWidget = SharedThis(this);

	const auto& SelectedItems = ListView->GetSelectedItems();
	for (const FChangelistTreeItemPtr& SelectedItem : SelectedItems)
	{
		const FFileTreeItem* FileItem = GetFileItem(SelectedItem);
		if (FileItem != nullptr)
		{
			USourceControlSubmitWidgetContext::SelectedItem ItemInfo;
			ItemInfo.FileName = FileItem->GetFileName().ToString();

			SourceControlSubmitWidgetContext->GetSelectedItems().Add(ItemInfo);
		}
	}

	FToolMenuContext Context;
	Context.AddObject(SourceControlSubmitWidgetContext);

	// Generate menu
	UToolMenu* Menu = ToolMenus->GenerateMenu(MenuName, Context);

	FToolMenuSection& Section = *Menu->FindSection("Source Control");
	if (SSourceControlSubmitWidget::CanDiffAgainstDepot())
	{
		Section.AddMenuEntry(
			NAME_None,
			NSLOCTEXT("SourceControl.SubmitWindow.Menu", "DiffAgainstDepot", "Diff Against Depot"),
			NSLOCTEXT("SourceControl.SubmitWindow.Menu", "DiffAgainstDepotTooltip", "Look at differences between your version of the asset and that in revision control."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.Actions.Diff"),
			FUIAction(
				FExecuteAction::CreateSP(this, &SSourceControlSubmitWidget::OnDiffAgainstDepot),
				FCanExecuteAction::CreateSP(this, &SSourceControlSubmitWidget::CanDiffAgainstDepot)
			)
		);
	}

	if (AllowRevert())
	{
		Section.AddMenuEntry(
			NAME_None,
			NSLOCTEXT("SourceControl.SubmitWindow.Menu", "Revert", "Revert"),
			NSLOCTEXT("SourceControl.SubmitWindow.Menu", "RevertTooltip", "Revert the selected assets to their original state from revision control."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.Actions.Revert"),
			FUIAction(
				FExecuteAction::CreateSP(this, &SSourceControlSubmitWidget::OnRevert),
				FCanExecuteAction::CreateSP(this, &SSourceControlSubmitWidget::CanRevert)
			)
		);
	}

	return ToolMenus->GenerateWidget(Menu);
}

bool SSourceControlSubmitWidget::CanDiffAgainstDepot() const
{
	bool bCanDiff = false;
	if (bAllowDiffAgainstDepot)
	{
		const auto& SelectedItems = ListView->GetSelectedItems();
		if (SelectedItems.Num() == 1)
		{
			bCanDiff = GetFileItem(SelectedItems[0])->CanDiff();
		}
	}
	return bCanDiff;
}

void SSourceControlSubmitWidget::OnDiffAgainstDepot()
{
	const auto& SelectedItems = ListView->GetSelectedItems();
	if (SelectedItems.Num() == 1)
	{
		OnDiffAgainstDepotSelected(SelectedItems[0]);
	}
}

void SSourceControlSubmitWidget::OnDiffAgainstDepotSelected(FChangelistTreeItemPtr InSelectedItem)
{
	if (bAllowDiffAgainstDepot)
	{
		FString PackageName;
		if (FPackageName::TryConvertFilenameToLongPackageName(GetFileItem(InSelectedItem)->GetFileName().ToString(), PackageName))
		{
			TArray<FAssetData> Assets;
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
			AssetRegistryModule.Get().GetAssetsByPackageName(*PackageName, Assets);
			if (Assets.Num() == 1)
			{
				const FAssetData& AssetData = Assets[0];
				UObject* CurrentObject = AssetData.GetAsset();
				if (CurrentObject)
				{
					const FString AssetName = AssetData.AssetName.ToString();
					FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
					AssetToolsModule.Get().DiffAgainstDepot(CurrentObject, PackageName, AssetName);
				}
			}
		}
	}
}

bool SSourceControlSubmitWidget::AllowRevert() const
{
	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("SourceControl.Revert.EnableFromSubmitWidget")))
	{
		return CVar->GetBool();
	}
	else
	{
		return false;
	}
}

bool SSourceControlSubmitWidget::CanRevert() const
{
	const auto& SelectedItems = ListView->GetSelectedItems();
	if (SelectedItems.Num() > 0)
	{
		return Algo::AllOf(SelectedItems, [](const FChangelistTreeItemPtr& SelectedItem)
			{
				return GetFileItem(SelectedItem)->CanRevert();
			}
		);
	}
	return false;
}

void SSourceControlSubmitWidget::OnRevert()
{
	const auto& SelectedItems = ListView->GetSelectedItems();
	if (SelectedItems.Num() < 1)
	{
		return;
	}

	auto RemoveItemsFromListView = [this](const TSet<FString>& ItemsToRemove)
	{
		ListViewItems.RemoveAll([&ItemsToRemove](const FChangelistTreeItemPtr& ListViewItem) -> bool
			{
				const FString FilePath = GetFileItem(ListViewItem)->GetFileName().ToString();
				return ItemsToRemove.Contains(FilePath);
			}
		);

		CheckedOutByOther.RemoveAll([&ItemsToRemove](const FSourceControlStateRef& StateItem) -> bool
			{
				const FString& FilePath = StateItem->GetFilename();
				return ItemsToRemove.Contains(FilePath);
			}
		);
	};

	TArray<FString> PackagesToRevert;
	TArray<FString> FilesToRevert;
	for (const FChangelistTreeItemPtr& SelectedItem : SelectedItems)
	{
		const FFileTreeItem* FileItem = GetFileItem(SelectedItem);
		if (FPackageName::IsPackageFilename(FileItem->GetFileName().ToString()))
		{
			PackagesToRevert.Add(FileItem->GetFileName().ToString());
		}
		else
		{
			FilesToRevert.Add(FileItem->GetFileName().ToString());
		}
	}

	{
		bool bAnyReverted = false;
		if (PackagesToRevert.Num() > 0)
		{
			bAnyReverted = SourceControlHelpers::RevertAndReloadPackages(PackagesToRevert, /*bRevertAll=*/false, /*bReloadWorld=*/true);
			RemoveItemsFromListView(TSet<FString>(PackagesToRevert));
		}
		if (FilesToRevert.Num() > 0)
		{
			bAnyReverted |= SourceControlHelpers::RevertFiles(FilesToRevert);
			RemoveItemsFromListView(TSet<FString>(FilesToRevert));
		}
		
		if (bAnyReverted)
		{
			if (ListViewItems.IsEmpty())
			{
				DialogResult = ESubmitResults::SUBMIT_CANCELED;
				ParentFrame.Pin()->RequestDestroyWindow();
			}
			else
			{
				ListView->RebuildList();
			}
		}
	}
}

FReply SSourceControlSubmitWidget::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent )
{
	// Pressing escape returns as if the user clicked cancel
	if (InKeyEvent.GetKey() == EKeys::Escape)
	{
		return CancelClicked();
	}

	return FReply::Unhandled();
}

void SSourceControlSubmitWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	ISourceControlModule::Get().Tick();

	FTSTicker::GetCoreTicker().Tick(InDeltaTime);

	// The user can release locks in-widget, so need to check for lock status changes.
	if (CheckedOutByOther.Num() > 0)
	{
		for (int32 Index = 0; Index < CheckedOutByOther.Num(); /* */)
		{
			const FSourceControlStateRef& State = CheckedOutByOther[Index];
			if (State->IsCheckedOutOther() && !State->CanCheckout())
			{
				++Index;
			}
			else
			{
				CheckedOutByOther.RemoveAtSwap(Index);
			}
		}
	}
}

ECheckBoxState SSourceControlSubmitWidget::GetToggleSelectedState() const
{
	int32 NumChecked = 0;
	for (const FChangelistTreeItemPtr& Item : ListViewItems)
	{
		switch (GetFileItem(Item)->GetCheckBoxState())
		{
		case ECheckBoxState::Checked:
			++NumChecked;
			break;
		case ECheckBoxState::Undetermined:
			return ECheckBoxState::Undetermined;
		}
	}

	if (NumChecked == 0)
	{
		return ECheckBoxState::Unchecked;
	}

	if (NumChecked == ListViewItems.Num())
	{
		return ECheckBoxState::Checked;
	}

	return ECheckBoxState::Undetermined;
}


void SSourceControlSubmitWidget::OnToggleSelectedCheckBox(ECheckBoxState InNewState)
{
	for (const FChangelistTreeItemPtr& Item : ListViewItems)
	{
		GetFileItem(Item)->SetCheckBoxState(InNewState);
	}

	ListView->RequestListRefresh();
}


void SSourceControlSubmitWidget::FillChangeListDescription(FChangeListDescription& OutDesc)
{
	OutDesc.Description = ChangeListDescriptionTextCtrl->GetText();

	OutDesc.FilesForAdd.Empty();
	OutDesc.FilesForSubmit.Empty();

	for (const FChangelistTreeItemPtr& Item : ListViewItems)
	{
		const FFileTreeItem* FileItem = GetFileItem(Item);
		if (FileItem->GetCheckBoxState() == ECheckBoxState::Checked)
		{
			if (FileItem->CanCheckIn())
			{
				OutDesc.FilesForSubmit.Add(FileItem->GetFileName().ToString());
			}
			else if (FileItem->NeedsAdding())
			{
				OutDesc.FilesForAdd.Add(FileItem->GetFileName().ToString());
			}
		}
	}
}


bool SSourceControlSubmitWidget::WantToKeepCheckedOut()
{
	return KeepCheckedOut == ECheckBoxState::Checked ? true : false;
}

void SSourceControlSubmitWidget::ClearChangeListDescription()
{
	ChangeListDescriptionTextCtrl->SetText(FText());
}

FReply SSourceControlSubmitWidget::SubmitClicked()
{
	DialogResult = ESubmitResults::SUBMIT_ACCEPTED;
	ParentFrame.Pin()->RequestDestroyWindow();

	return FReply::Handled();
}

FReply SSourceControlSubmitWidget::CancelClicked()
{
	DialogResult = ESubmitResults::SUBMIT_CANCELED;
	ParentFrame.Pin()->RequestDestroyWindow();

	return FReply::Handled();
}

FReply SSourceControlSubmitWidget::SaveAndCloseClicked()
{
	DialogResult = ESubmitResults::SUBMIT_SAVED;
	ParentFrame.Pin()->RequestDestroyWindow();

	return FReply::Handled();
}

bool SSourceControlSubmitWidget::IsSubmitEnabled() const
{
	return bAllowSubmit && !ChangeListDescriptionTextCtrl->GetText().ToString().TrimStartAndEnd().IsEmpty() && ListViewItems.Num() > 0 && CheckedOutByOther.Num() == 0;
}


EVisibility SSourceControlSubmitWidget::IsWarningPanelVisible() const
{
	return IsSubmitEnabled() ? EVisibility::Collapsed : EVisibility::Visible;
}

bool SSourceControlSubmitWidget::IsValidationWarningVisible() const
{
	const bool bValidationWarningVisible = !bAllowSubmit;
	return bValidationWarningVisible;
}

bool SSourceControlSubmitWidget::IsCheckoutWarningVisible() const
{
	const bool bValidationWarningVisible = IsValidationWarningVisible();
	const bool bCheckoutWarningVisible = CheckedOutByOther.Num() > 0;
	return bCheckoutWarningVisible && !bValidationWarningVisible;
}

bool SSourceControlSubmitWidget::IsDescriptionWarningVisible() const
{
	const bool bValidationWarningVisible = IsValidationWarningVisible();
	const bool bCheckoutWarningVisible = IsCheckoutWarningVisible();
	const bool bDescriptionWarningVisible = ChangeListDescriptionTextCtrl->GetText().ToString().TrimStartAndEnd().IsEmpty();
	return bDescriptionWarningVisible && !bValidationWarningVisible && !bCheckoutWarningVisible;
}




void SSourceControlSubmitWidget::OnCheckStateChanged_KeepCheckedOut(ECheckBoxState InState)
{
	KeepCheckedOut = InState;
}


ECheckBoxState SSourceControlSubmitWidget::GetKeepCheckedOut() const
{
	return KeepCheckedOut;
}


bool SSourceControlSubmitWidget::CanCheckOut() const
{
	const ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	return SourceControlProvider.UsesCheckout();
}


TSharedRef<ITableRow> SSourceControlSubmitWidget::OnGenerateRowForList(FChangelistTreeItemPtr SubmitItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SFileTableRow, OwnerTable)
		.TreeItemToVisualize(SubmitItem)
		.PathFlags(bShowingContentVersePath ? SourceControlFileViewColumn::EPathFlags::ShowingVersePath : SourceControlFileViewColumn::EPathFlags::Default);
}


EColumnSortMode::Type SSourceControlSubmitWidget::GetColumnSortMode(const FName ColumnId) const
{
	if (SortByColumn != ColumnId)
	{
		return EColumnSortMode::None;
	}

	return SortMode;
}


void SSourceControlSubmitWidget::OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	SortByColumn = ColumnId;
	SortMode = InSortMode;

	RequestSort();
}


EVisibility SSourceControlSubmitWidget::GetIconColumnContentVisibility() const
{
	// Hide the icon when sorting the icon column (it clashes with the sort mode icon).
	return GetColumnSortMode(SourceControlFileViewColumn::Icon::Id()) == EColumnSortMode::None ? EVisibility::Visible : EVisibility::Collapsed;
}


void SSourceControlSubmitWidget::RequestSort()
{
	// Sort the list of root items
	SortTree();

	ListView->RequestListRefresh();
}


void SSourceControlSubmitWidget::SortTree()
{
	TFunction<bool(const IFileViewTreeItem&, const IFileViewTreeItem&)> SortPredicate = SourceControlFileViewColumn::GetSortPredicate(
		SortMode, SortByColumn, bShowingContentVersePath ? SourceControlFileViewColumn::EPathFlags::ShowingVersePath : SourceControlFileViewColumn::EPathFlags::Default);
	if (SortPredicate)
	{
		Algo::SortBy(
			ListViewItems,
			[](const FChangelistTreeItemPtr& ListViewItem) -> const IFileViewTreeItem&
			{
				return *GetFileItem(ListViewItem);
			},
			SortPredicate);
	}
}

FFileTreeItem* SSourceControlSubmitWidget::GetFileItem(const FChangelistTreeItemPtr& ChangelistItem)
{
	check(ChangelistItem->GetTreeItemType() == IChangelistTreeItem::File);
	return static_cast<FFileTreeItem*>(ChangelistItem.Get());
}

#undef LOCTEXT_NAMESPACE

#endif // SOURCE_CONTROL_WITH_SLATE
