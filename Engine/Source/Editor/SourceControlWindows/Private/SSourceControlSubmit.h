// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "ISourceControlState.h"
#include "SSourceControlCommon.h"

class SMultiLineEditableTextBox;
class SWindow;

//-------------------------------------
// Source Control Window Constants
//-------------------------------------
namespace ESubmitResults
{
	enum Type
	{
		SUBMIT_ACCEPTED,
		SUBMIT_CANCELED,
		SUBMIT_SAVED,
	};
}

//-----------------------------------------------
// Source Control Check in Helper Struct
//-----------------------------------------------
class FChangeListDescription
{
public:
	TArray<FString> FilesForAdd;
	TArray<FString> FilesForSubmit;
	FText Description;
};

bool TryToVirtualizeFilesToSubmit(const TArray<FString>& FilesToSubmit, FText& Description, FText& OutFailureMsg);
bool TryToVirtualizeFilesToSubmit(const TArray<FSourceControlStateRef>& FileStates, FText& Description, FText& OutFailureMsg);

DECLARE_DELEGATE_OneParam(FSourceControlSaveChangelistDescription, const FText& /*NewDescription*/);

class SSourceControlSubmitWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSourceControlSubmitWidget)
		: _ParentWindow()
		, _Items()
		, _Description()
		, _ChangeValidationResult()
		, _ChangeValidationWarnings()
		, _ChangeValidationErrors()
		, _AllowDescriptionChange(true)
		, _AllowUncheckFiles(true)
		, _AllowKeepCheckedOut(true)
		, _AllowSubmit(true)
		, _AllowSaveAndClose(false)
		, _AllowDiffAgainstDepot(true)
	{}

		SLATE_ATTRIBUTE(TSharedPtr<SWindow>, ParentWindow)
		SLATE_ATTRIBUTE(TArray<FSourceControlStateRef>, Items)
		SLATE_ATTRIBUTE(FText, Description)
		SLATE_ATTRIBUTE(FString, ChangeValidationResult)
		SLATE_ATTRIBUTE(FString, ChangeValidationWarnings)
		SLATE_ATTRIBUTE(FString, ChangeValidationErrors)
		SLATE_ATTRIBUTE(bool, AllowDescriptionChange)
		SLATE_ATTRIBUTE(bool, AllowUncheckFiles)
		SLATE_ATTRIBUTE(bool, AllowKeepCheckedOut)
		SLATE_ATTRIBUTE(bool, AllowSubmit)
		SLATE_ATTRIBUTE(bool, AllowSaveAndClose)
		SLATE_ATTRIBUTE(bool, AllowDiffAgainstDepot)

	SLATE_END_ARGS()

	~SSourceControlSubmitWidget();

	/** Constructs the widget */
	void Construct(const FArguments& InArgs);

	/** Used to intercept Escape key press, and interpret it as cancel */
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;

	/** Used to tick the AssetDataCache via the SourceControlModule when the dialog is open to refresh asset names */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/** Get dialog result */
	ESubmitResults::Type GetResult() { return DialogResult; }

	/** Gets the requested files and the change list description*/
	void FillChangeListDescription(FChangeListDescription& OutDesc);

	/** Does the user want to keep the files checked out */
	bool WantToKeepCheckedOut();

	/** Clears the current change list description */
	void ClearChangeListDescription();

private:
	/**
	 * @return the desired toggle state for the ToggleSelectedCheckBox.
	 * Returns Unchecked, unless all of the selected items are Checked.
	 */
	ECheckBoxState GetToggleSelectedState() const;

	/**
	 * Toggles the highlighted items.
	 * If no items are explicitly highlighted, toggles all items in the list.
	 */
	void OnToggleSelectedCheckBox(ECheckBoxState InNewState);

	/** Called when the settings of the dialog are to be accepted*/
	FReply SubmitClicked();

	/** Called when the settings of the dialog are to be ignored*/
	FReply CancelClicked();

	/** Called when the user click the 'Save' button. */
	FReply SaveAndCloseClicked();

	/** Called to check if the submit button is enabled or not. */
	bool IsSubmitEnabled() const;

	/** Check if the warning panel should be visible. */
	EVisibility IsWarningPanelVisible() const;

	/** Check if the warning should be visible in the warning panel. */
	bool IsValidationWarningVisible() const;
	bool IsCheckoutWarningVisible() const;
	bool IsDescriptionWarningVisible() const;

	/** Called when the Keep checked out Checkbox is changed */
	void OnCheckStateChanged_KeepCheckedOut(ECheckBoxState InState);

	/** Get the current state of the Keep Checked Out checkbox  */
	ECheckBoxState GetKeepCheckedOut() const;

	/** Check if Provider can checkout files */
	bool CanCheckOut() const;

	/** Called by SListView to get a widget corresponding to the supplied item */
	TSharedRef<ITableRow> OnGenerateRowForList(FChangelistTreeItemPtr SubmitItemData, const TSharedRef<STableViewBase>& OwnerTable);

	/**
	 * Returns the current column sort mode (ascending or descending) if the ColumnId parameter matches the current
	 * column to be sorted by, otherwise returns EColumnSortMode_None.
	 *
	 * @param	ColumnId	Column ID to query sort mode for.
	 *
	 * @return	The sort mode for the column, or EColumnSortMode_None if it is not known.
	 */
	EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;

	/**
	 * Callback for SHeaderRow::Column::OnSort, called when the column to sort by is changed.
	 *
	 * @param	ColumnId	The new column to sort by
	 * @param	InSortMode	The sort mode (ascending or descending)
	 */
	void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);

	EVisibility GetIconColumnContentVisibility() const;

	/**
	 * Requests that the source list data be sorted according to the current sort column and mode,
	 * and refreshes the list view.
	 */
	void RequestSort();

	/**
	 * Sorts the source list data according to the current sort column and mode.
	 */
	void SortTree();

	TSharedPtr<SWidget> OnCreateContextMenu();

	bool CanDiffAgainstDepot() const;
	void OnDiffAgainstDepot();
	void OnDiffAgainstDepotSelected(FChangelistTreeItemPtr InSelectedItem);

	/** Called to check whether the selected files in the ListView can be reverted */
	bool AllowRevert() const;
	bool CanRevert() const;

	/** Reverts the files selected in the ListView */
	void OnRevert();

	static FFileTreeItem* GetFileItem(const FChangelistTreeItemPtr& ChangelistItem);

private:
	ESubmitResults::Type DialogResult;

	/** ListBox for selecting which object to consolidate */
	TSharedPtr<SListView<FChangelistTreeItemPtr>> ListView;

	/** Collection of objects (Widgets) to display in the List View. */
	TArray<FChangelistTreeItemPtr> ListViewItems;

	/** Collection of states that are checked out by other users. */
	TArray<FSourceControlStateRef> CheckedOutByOther;

	/** Pointer to the parent modal window */
	TWeakPtr<SWindow> ParentFrame;

	/** Internal widgets to save having to get in multiple places*/
	TSharedPtr<SMultiLineEditableTextBox> ChangeListDescriptionTextCtrl;

	/** State of the "Keep checked out" checkbox */
	ECheckBoxState	KeepCheckedOut;

	/** Whether the submit button should be enabled or not */
	bool bAllowSubmit;

	/** Whether a diff against the depot may be performed from within the submit dialog */
	bool bAllowDiffAgainstDepot;

	/** Whether the content verse path should be shown or not */
	bool bShowingContentVersePath;

	/** Specify which column to sort with */
	FName SortByColumn;

	/** Currently selected sorting mode */
	EColumnSortMode::Type SortMode;
};
