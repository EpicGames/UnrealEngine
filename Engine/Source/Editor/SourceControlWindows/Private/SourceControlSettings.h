// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "SourceControlSettings.generated.h"

/** Serializes source control window settings. */
UCLASS(config=Editor)
class USourceControlSettings : public UObject
{
	GENERATED_BODY()

public:
	USourceControlSettings() {}

	UPROPERTY(config, EditAnywhere, Category="Revision Control Changelist View", meta = (Keywords = "Source Control"))
	bool bShowAssetTypeColumn = true;

	UPROPERTY(config, EditAnywhere, Category="Revision Control Changelist View", meta = (Keywords = "Source Control"))
	bool bShowAssetLastModifiedTimeColumn = true;

	UPROPERTY(config, EditAnywhere, Category="Revision Control Changelist View", meta = (Keywords = "Source Control"))
	bool bShowAssetCheckedOutByColumn = true;
};
