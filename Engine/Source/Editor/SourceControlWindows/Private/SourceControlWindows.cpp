// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceControlWindows.h"
#include "SSourceControlSubmit.h"
#include "SSourceControlCheckedOutDialog.h"
#include "AssetViewUtils.h"
#include "FileHelpers.h"
#include "ISourceControlModule.h"
#include "ISourceControlWindowsModule.h"
#include "SourceControlHelpers.h"
#include "SourceControlOperations.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Logging/MessageLog.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "SourceControlSettings.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SourceControlWindows)

#if SOURCE_CONTROL_WITH_SLATE

#define LOCTEXT_NAMESPACE "SourceControlWindows"

namespace
{

static FAutoConsoleCommand CCmdRunChoosePackagesToCheckIn = FAutoConsoleCommand(
	TEXT("RevisionControl.ChoosePackagesToCheckIn"),
	TEXT("Display all content changes in the current project and allow the user to select which changes to submit to revision control"),
	FConsoleCommandDelegate::CreateLambda([]()
		{
			FSourceControlWindows::ChoosePackagesToCheckIn();
		}));

static bool GEnableFilesValidation = true;
static TAutoConsoleVariable<bool> CVarEnableFilesValidation(
	TEXT("RevisionControl.FilesValidation.Enable"),
	GEnableFilesValidation,
	TEXT("Enables pre-submit validation of files."));

static bool GRequireFilesValidation = true;
static TAutoConsoleVariable<bool> CVarRequireFilesValidation(
	TEXT("RevisionControl.FilesValidation.Require"),
	GRequireFilesValidation,
	TEXT("Requires pre-submit validation of files."));

} // anonymous namespace

//---------------------------------------------------------------------------------------
// FCheckinResultInfo

FCheckinResultInfo::FCheckinResultInfo()
	: Result(ECommandResult::Failed)
	, bAutoCheckedOut(false)
{
}


//---------------------------------------------------------------------------------------
// Helper function(s)

static bool SaveDirtyPackages(bool bUseDialog)
{
	const bool bPromptUserToSave = bUseDialog;
	const bool bSaveMapPackages = true;
	const bool bSaveContentPackages = true;
	const bool bFastSave = false;
	const bool bNotifyNoPackagesSaved = false;
	const bool bCanBeDeclined = true; // If the user clicks "don't save" this will continue and lose their changes

	bool bSaved = FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages, bFastSave, bNotifyNoPackagesSaved, bCanBeDeclined);

	// bSaved can be true if the user selects to not save an asset by unchecking it and clicking "save"
	if (bSaved)
	{
		bSaved = !FEditorFileUtils::HasDirtyPackages();
	}

	return bSaved;
}

static bool GetFilesValidationResult(TArray<FSourceControlStateRef>& InStates, FString& OutValidationTitleText, FString& OutValidationWarningsText, FString& OutValidationErrorsText)
{
	FSourceControlPreSubmitFilesValidationExDelegate ValidationDelegate = ISourceControlModule::Get().GetRegisteredPreSubmitFilesValidationEx();

	FSourceControlValidationDelegateResult ValidationResult;

	bool bValidationResult = true;

	if (ValidationDelegate.ExecuteIfBound(InStates, ValidationResult))
	{
		EMessageSeverity::Type MessageSeverity = EMessageSeverity::Info;

		if (ValidationResult.Result == EDataValidationResult::Invalid || ValidationResult.Errors.Num() > 0)
		{
			OutValidationTitleText = LOCTEXT("SourceControl.Submit.FilesValidationError", "File(s) validation failed!").ToString();
			bValidationResult = false;
			MessageSeverity = EMessageSeverity::Error;
		}
		else if (ValidationResult.Result == EDataValidationResult::NotValidated || ValidationResult.Warnings.Num() > 0)
		{
			OutValidationTitleText = LOCTEXT("SourceControl.Submit.FilesValidationWarning", "File(s) validation has warnings!").ToString();
			MessageSeverity = EMessageSeverity::Warning;
		}
		else
		{
			OutValidationTitleText = LOCTEXT("SourceControl.Submit.FilesValidationSuccess", "File(s) validation successful!").ToString();
		}

		auto AppendInfo = [](const TArray<FText>& Info, const FText& HeaderFormat, const FText& OverflowFormat, FString& OutText)
			{
				const int32 MaxNumLinesDisplayed = 5;
				int32 NumLinesDisplayed = 0;

				if (Info.Num() > 0)
				{
					if (OutText.Len() > 0)
					{
						OutText += LINE_TERMINATOR;
					}

					FFormatNamedArguments Args;
					Args.Add(TEXT("Count"), Info.Num());
					OutText += FText::Format(HeaderFormat, Args).ToString();

					for (const FText& Line : Info)
					{
						if (NumLinesDisplayed >= MaxNumLinesDisplayed)
						{
							OutText += LINE_TERMINATOR;
							OutText += OverflowFormat.ToString();
							break;
						}

						OutText += LINE_TERMINATOR;
						OutText += Line.ToString();

						++NumLinesDisplayed;
					}
				}
			};

		AppendInfo(ValidationResult.Errors, LOCTEXT("ValidationErrorsHeader", "Encountered {Count} {Count}|plural(one=error,other=errors):"), LOCTEXT("ValidationErrorsOverflow", "See log for complete list of errors"), OutValidationErrorsText);
		AppendInfo(ValidationResult.Warnings, LOCTEXT("ValidationWarningsHeader", "Encountered {Count} {Count}|plural(one=warning,other=warnings):"), LOCTEXT("ValidationWarningsOverflow", "See log for complete list of warnings"), OutValidationWarningsText);
	}

	return bValidationResult;
}

//---------------------------------------------------------------------------------------
// FSourceControlWindows

TWeakPtr<SNotificationItem> FSourceControlWindows::ChoosePackagesToCheckInNotification;

bool FSourceControlWindows::ChoosePackagesToCheckIn(const FSourceControlWindowsOnCheckInComplete& OnCompleteDelegate)
{
	if (!ISourceControlModule::Get().IsEnabled())
	{
		FCheckinResultInfo ResultInfo;
		ResultInfo.Description = LOCTEXT("SourceControlDisabled", "Revision control is not enabled.");
		OnCompleteDelegate.ExecuteIfBound(ResultInfo);

		return false;
	}

	if (!ISourceControlModule::Get().GetProvider().IsAvailable())
	{
		FCheckinResultInfo ResultInfo;
		ResultInfo.Description = LOCTEXT("NoSCCConnection", "No connection to revision control available!");

		FMessageLog EditorErrors("EditorErrors");
		EditorErrors.Warning(ResultInfo.Description)->AddToken(
			FDocumentationToken::Create(TEXT("Engine/UI/SourceControl")));
		EditorErrors.Notify();

		OnCompleteDelegate.ExecuteIfBound(ResultInfo);

		return false;
	}

	if (ISourceControlModule::Get().GetProvider().UsesSnapshots())
	{
		SaveDirtyPackages(/*bUseDialog=*/false);
	}

	// Start selection process...

	// make sure we update the SCC status of all packages (this could take a long time, so we will run it as a background task)
	TArray<FString> Filenames = SourceControlHelpers::GetSourceControlLocations();
	
	// make sure the SourceControlProvider state cache is populated as well
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	FSourceControlOperationRef Operation = ISourceControlOperation::Create<FUpdateStatus>();
	SourceControlProvider.Execute(
		Operation,
		Filenames,
		EConcurrency::Asynchronous,
		FSourceControlOperationComplete::CreateStatic(&FSourceControlWindows::ChoosePackagesToCheckInCallback, OnCompleteDelegate));

	if (ChoosePackagesToCheckInNotification.IsValid())
	{
		ChoosePackagesToCheckInNotification.Pin()->ExpireAndFadeout();
	}

	FNotificationInfo Info(LOCTEXT("ChooseAssetsToCheckInIndicator", "Checking for assets to check in..."));
	Info.bFireAndForget = false;
	Info.ExpireDuration = 0.0f;
	Info.FadeOutDuration = 1.0f;

	if (SourceControlProvider.CanCancelOperation(Operation))
	{
		Info.ButtonDetails.Add(FNotificationButtonInfo(
			LOCTEXT("ChoosePackagesToCheckIn_CancelButton", "Cancel"),
			LOCTEXT("ChoosePackagesToCheckIn_CancelButtonTooltip", "Cancel the check in operation."),
			FSimpleDelegate::CreateStatic(&FSourceControlWindows::ChoosePackagesToCheckInCancelled, Operation)
			));
	}

	ChoosePackagesToCheckInNotification = FSlateNotificationManager::Get().AddNotification(Info);

	if (ChoosePackagesToCheckInNotification.IsValid())
	{
		ChoosePackagesToCheckInNotification.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
	}

	return true;
}


bool FSourceControlWindows::CanChoosePackagesToCheckIn()
{
	ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
	
	if (ISourceControlModule::Get().IsEnabled() &&
		ISourceControlModule::Get().GetProvider().IsAvailable() &&
		!ChoosePackagesToCheckInNotification.IsValid())
	{
		TOptional<bool> bHasChangesToCheckIn = SourceControlModule.GetProvider().HasChangesToCheckIn();
		if (bHasChangesToCheckIn.IsSet())
		{
			return bHasChangesToCheckIn.GetValue();
		}
		else
		{
			return true;
		}
	}
 
	return false;
}

bool FSourceControlWindows::SyncLatest()
{
	return SyncRevision(TEXT(""));
}

bool FSourceControlWindows::SyncRevision(const FString& InRevision)
{
	bool bSaved = SaveDirtyPackages(/*bUseDialog=*/true);

	// if not properly saved, ask for confirmation from the user before continuing.
	if (!bSaved)
	{
		FText DialogText = NSLOCTEXT("SourceControlCommands", "UnsavedWarningText", "Warning: There are modified assets which are not being saved. If you sync to latest you may lose your unsaved changes. Do you want to continue?");
		FText DialogTitle = NSLOCTEXT("SourceControlCommands", "UnsavedWarningTitle", "Unsaved changes");

		EAppReturnType::Type DialogResult = FMessageDialog::Open(EAppMsgType::YesNo, DialogText, DialogTitle);

		bSaved = (DialogResult == EAppReturnType::Yes);
	}

	// if properly saved or confirmation given, find all packages and use source control to update them.
	if (bSaved)
	{
		bool bSuccess = AssetViewUtils::SyncRevisionFromSourceControl(InRevision);
		if (!bSuccess)
		{
			FText Message(LOCTEXT("SCC_Sync_Failed", "Failed to sync files!"));
			FMessageLog("SourceControl").Notify(Message);
		}
		return bSuccess;
	}

	return false;
}


bool FSourceControlWindows::CanSyncLatest()
{
	ISourceControlModule& SourceControlModule = ISourceControlModule::Get();

	if (SourceControlModule.IsEnabled() &&
		SourceControlModule.GetProvider().IsAvailable())
	{
		TOptional<bool> bHasChangesToSync = SourceControlModule.GetProvider().HasChangesToSync();
		if (bHasChangesToSync.IsSet())
		{
			return bHasChangesToSync.GetValue();
		}
		else
		{
			return true;
		}
	}

	return false;
}


bool FSourceControlWindows::PromptForCheckin(FCheckinResultInfo& OutResultInfo, const TArray<FString>& InPackageNames, const TArray<FString>& InPendingDeletePaths, const TArray<FString>& InConfigFiles, bool bUseSourceControlStateCache)
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	const bool bUsesSnapshots = SourceControlProvider.UsesSnapshots();
	const bool bUsesChangelists = SourceControlProvider.UsesChangelists();
	const bool bUsesCheckout = SourceControlProvider.UsesCheckout();

	//~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	// Get filenames for packages and config to be checked in
	TArray<FString> AllFiles = SourceControlHelpers::PackageFilenames(InPackageNames);
	AllFiles.Append(InConfigFiles);

	// Prepare a list of files to have their states updated
	if (!bUseSourceControlStateCache)
	{
		TArray<FString> UpdateRequest;
		UpdateRequest.Append(AllFiles);

		// If there are pending delete paths to update, add them here.
		UpdateRequest.Append(InPendingDeletePaths);

		// Force an update on everything that's been requested
		if (UpdateRequest.Num() > 0)
		{
			SourceControlProvider.Execute(ISourceControlOperation::Create<FUpdateStatus>(), UpdateRequest);
		}
	}

	// Get file status of packages and config
	TArray<FSourceControlStateRef> States;
	SourceControlProvider.GetState(AllFiles, States, EStateCacheUsage::Use);

	if (InPendingDeletePaths.Num() > 0)
	{
		// Get any files pending delete
		TArray<FSourceControlStateRef> PendingDeleteItems = SourceControlProvider.GetCachedStateByPredicate(
			[&States](const FSourceControlStateRef& State) 
			{ 
				return State->IsDeleted() 
					// if the states already contains the pending delete do not bother appending it
					&& !States.Contains(State); 
			}
		);

		// And append them to the list
		States.Append(PendingDeleteItems);
	}

	//~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	// Exit if no assets needing check in
	if (States.Num() == 0)
	{
		OutResultInfo.Result      = ECommandResult::Succeeded;
		OutResultInfo.Description = LOCTEXT("NoAssetsToCheckIn", "No assets to check in!");

		FMessageLog EditorErrors("EditorErrors");
		EditorErrors.Warning(OutResultInfo.Description);
		EditorErrors.Notify();

		// Consider it a success even if no files were checked in
		return true;
	}

	//~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	// Validate assets that need check in
	const bool bValidationEnabled = CVarEnableFilesValidation.GetValueOnGameThread();
	const bool bValidationRequired = CVarRequireFilesValidation.GetValueOnGameThread() && (bUsesSnapshots || bUsesChangelists);

	FString ChangelistValidationTitle;
	FString ChangelistValidationWarningsText;
	FString ChangelistValidationErrorsText;
	const bool bValidationResult = bValidationEnabled
		? GetFilesValidationResult(States, ChangelistValidationTitle, ChangelistValidationWarningsText, ChangelistValidationErrorsText)
		: true;

	//~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	// Generate the submit files window title
	FSubmitTitleOverrideParameters SubmitTitleOverrideParameters;
	SubmitTitleOverrideParameters.ToSubmit.SetSubtype<TArray<FString>>(AllFiles);

	const FText Title = ISourceControlWindowsModule::Get().SubmitTitleOverrideDelegate.IsBound()
		? ISourceControlWindowsModule::Get().SubmitTitleOverrideDelegate.Execute(SubmitTitleOverrideParameters)
		: NSLOCTEXT("SourceControl.ConfirmSubmit", "Title", "Confirm Submit");

	//~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	// Create a submit files window
	TSharedRef<SWindow> NewWindow = SNew(SWindow)
		.Title(Title)
		.SizingRule(ESizingRule::UserSized)
		.ClientSize(FVector2D(800, 700))
		.SupportsMaximize(true)
		.SupportsMinimize(false);

	const bool bAllowUncheckFiles = !bUsesSnapshots && !bUsesChangelists;
	const bool bAllowDiffAgainstDepot = SourceControlProvider.AllowsDiffAgainstDepot();
	const bool bAllowKeepCheckedOut = bUsesCheckout;

	TSharedRef<SSourceControlSubmitWidget> SourceControlWidget =
		SNew(SSourceControlSubmitWidget)
		.ParentWindow(NewWindow)
		.Items(States)
		.AllowUncheckFiles(bAllowUncheckFiles)
		.AllowDiffAgainstDepot(bAllowDiffAgainstDepot)
		.AllowSubmit(bValidationRequired ? bValidationResult : true)
		.AllowKeepCheckedOut(bAllowKeepCheckedOut)
		.ChangeValidationResult(ChangelistValidationTitle)
		.ChangeValidationWarnings(ChangelistValidationWarningsText)
		.ChangeValidationErrors(ChangelistValidationErrorsText);

	NewWindow->SetContent(
		SourceControlWidget
		);

	FSlateApplication::Get().AddModalWindow(NewWindow, NULL);


	//~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	// Exit if cancelled by user
	if (SourceControlWidget->GetResult() == ESubmitResults::SUBMIT_CANCELED)
	{
		OutResultInfo.Result      = ECommandResult::Cancelled;
		OutResultInfo.Description = LOCTEXT("CheckinCancelled", "File check in cancelled.");

		return false;
	}

	//~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	// Get description from the dialog
	FChangeListDescription Description;
	SourceControlWidget->FillChangeListDescription(Description);


	//~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	// Revert any unchanged files
	if (Description.FilesForSubmit.Num() > 0)
	{
		SourceControlHelpers::RevertUnchangedFiles(SourceControlProvider, Description.FilesForSubmit);

		if (ISourceControlModule::Get().GetCustomProjects().IsEmpty())
		{
			// Make sure all files are still checked out
			for (int32 VerifyIndex = Description.FilesForSubmit.Num() - 1; VerifyIndex >= 0; --VerifyIndex)
			{
				FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(Description.FilesForSubmit[VerifyIndex], EStateCacheUsage::Use);
				if (SourceControlState.IsValid() && !SourceControlState->IsCheckedOut() && !SourceControlState->IsAdded() && !SourceControlState->IsDeleted())
				{
					Description.FilesForSubmit.RemoveAt(VerifyIndex);
				}
			}
		}
		else
		{
			// For project-based source control, we want to go through with a check in attempt even when 
			// files are not checked out by the current user, and generate a warning dialog
		}
	}

	bool bSuccess = true; // Overall success
	
	TArray<FString> CombinedFileList;
	CombinedFileList.Append(Description.FilesForAdd);
	CombinedFileList.Append(Description.FilesForSubmit);

	//~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	// Any files to check in?
	if (CombinedFileList.Num() == 0)
	{
		OutResultInfo.Result = ECommandResult::Succeeded;

		if (OutResultInfo.Description.IsEmpty())
		{
			OutResultInfo.Description = LOCTEXT("SCC_No_Files", "No files were selected to check in to version control.");
		}

		return true;
	}

	//~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	// Mark files for add as needed
	if (Description.FilesForAdd.Num() > 0)
	{
		OutResultInfo.FilesAdded = Description.FilesForAdd;

		const bool bAddSuccess = SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), Description.FilesForAdd) == ECommandResult::Succeeded;
		if (!bAddSuccess)
		{
			// Note that this message may be overwritten with a checkin error below.
			OutResultInfo.Description = LOCTEXT("SCC_Add_Files_Error", "One or more files were not able to be marked for add to version control!");
		}
		
		bSuccess &= bAddSuccess;
	}

	//~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	// First check if there is a submit override bound
	if (ISourceControlWindowsModule::Get().SubmitOverrideDelegate.IsBound())
	{
		SSubmitOverrideParameters SubmitOverrideParameters;
		SubmitOverrideParameters.Description = Description.Description.ToString();
		SubmitOverrideParameters.ToSubmit.SetSubtype<TArray<FString>>(CombinedFileList);

		FSubmitOverrideReply SubmitOverrideReply = ISourceControlWindowsModule::Get().SubmitOverrideDelegate.Execute(SubmitOverrideParameters);
		switch (SubmitOverrideReply)
		{
			//////////////////////////////////////////////////////////
			case FSubmitOverrideReply::Handled:
			{
				OutResultInfo.Result = ECommandResult::Succeeded;
				OutResultInfo.Description = LOCTEXT("SCC_Checkin_SubmitOverride_Succeeded", "Successfully invoked the submit override!");
				return true;
			}

			//////////////////////////////////////////////////////////
			case FSubmitOverrideReply::Error:
			{
				OutResultInfo.Result = ECommandResult::Failed;
				OutResultInfo.Description = LOCTEXT("SCC_Checkin_SubmitOverride_Failed", "Failed to invoke the submit override!");
				return false;
			}
			
			//////////////////////////////////////////////////////////
			case FSubmitOverrideReply::ProviderNotSupported:
			default:
				// continue default flow
				break;
		}
	}

	//~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	// Virtualize the files that need to be submitted
	FText VirtualizationFailureMsg;
	if (!TryToVirtualizeFilesToSubmit(CombinedFileList, Description.Description, VirtualizationFailureMsg))
	{
		FMessageLog("SourceControl").Notify(VirtualizationFailureMsg);

		OutResultInfo.Result = ECommandResult::Failed;
		OutResultInfo.Description = VirtualizationFailureMsg;

		return false;
	}

	bool bCheckinSuccess = true;
	bool bCheckinCancelled = false;
	bool bSyncLatestNeeded = bUsesSnapshots ? FSourceControlWindows::CanSyncLatest() : false;

	FText CheckinSuccessMessage;

	do
	{
		//~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
		// Sync to latest if using snapshots as we cannot check in unless we're at head
		if (bSyncLatestNeeded)
		{
			if (!FSourceControlWindows::SyncLatest())
			{
				OutResultInfo.Description = LOCTEXT("SCC_Checkin_Aborted_Sync", "File check in aborted because the sync to the latest snapshot failed.");
				return false;
			}

			TArray<FSourceControlStateRef> Conflicts = ISourceControlModule::Get().GetProvider().GetCachedStateByPredicate(
				[](const FSourceControlStateRef& State)
				{
					return State->IsConflicted();
				}
			);

			const bool bConflictsRemaining = (Conflicts.Num() > 0);
			if (bConflictsRemaining)
			{
				OutResultInfo.Description = LOCTEXT("SCC_Checkin_Aborted_Conflicts", "File check in aborted because the sync to the latest snapshot resulted in conflicts that need to be resolved.");
				return false;
			}
		}

		//~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
		// Check in files
		TSharedRef<FCheckIn, ESPMode::ThreadSafe> CheckInOperation = ISourceControlOperation::Create<FCheckIn>();
		CheckInOperation->SetDescription(Description.Description);
		CheckInOperation->SetKeepCheckedOut(SourceControlWidget->WantToKeepCheckedOut());

		ECommandResult::Type CheckInResult = SourceControlProvider.Execute(CheckInOperation, CombinedFileList);
		bCheckinSuccess = CheckInResult == ECommandResult::Succeeded;
		bCheckinCancelled = CheckInResult == ECommandResult::Cancelled;

		//~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
		// Check if another user beat us to snapshot creation
		// This can happen - for example - when taking a long time writing a description
		bSyncLatestNeeded = bUsesSnapshots ? FSourceControlWindows::CanSyncLatest() : false;

		//~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
		// Store the success message for later use
		CheckinSuccessMessage = CheckInOperation->GetSuccessMessage();

	} while ((bUsesSnapshots && bSyncLatestNeeded) && !bCheckinSuccess && !bCheckinCancelled);

	bSuccess &= bCheckinSuccess;

	if (bCheckinSuccess)
	{
		// report success with a notification
		FNotificationInfo Info(CheckinSuccessMessage);
		Info.ExpireDuration = 8.0f;
		Info.HyperlinkText = LOCTEXT("SCC_Checkin_ShowLog", "Show Message Log");
		Info.Hyperlink = FSimpleDelegate::CreateStatic([](){ FMessageLog("SourceControl").Open(EMessageSeverity::Info, true); });
		FSlateNotificationManager::Get().AddNotification(Info);

		// also add to the log
		FMessageLog("SourceControl").Info(CheckinSuccessMessage);

		OutResultInfo.Result         = ECommandResult::Succeeded;
		OutResultInfo.Description    = CheckinSuccessMessage;
		OutResultInfo.FilesSubmitted = Description.FilesForSubmit;
	}

	//~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	// Abort if cancelled
	if (bCheckinCancelled)
	{
		FText Message(LOCTEXT("CheckinCancelled", "File check in cancelled."));

		OutResultInfo.Result      = ECommandResult::Cancelled;
		OutResultInfo.Description = Message;

		return false;
	}
	
	//~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~~
	// Exit if errors
	if (!bSuccess)
	{
		FText Message(LOCTEXT("SCC_Checkin_Failed", "Failed to check in files!"));
		FMessageLog("SourceControl").Notify(Message);

		OutResultInfo.Result = ECommandResult::Failed;

		if (!bCheckinSuccess || OutResultInfo.Description.IsEmpty())
		{
			OutResultInfo.Description = Message;
		}

		return false;
	}

	SourceControlWidget->ClearChangeListDescription();

	return true;
}


bool FSourceControlWindows::PromptForCheckin(bool bUseSourceControlStateCache, const TArray<FString>& InPackageNames, const TArray<FString>& InPendingDeletePaths, const TArray<FString>& InConfigFiles)
{
	FCheckinResultInfo ResultInfo;

	return PromptForCheckin(ResultInfo, InPackageNames, InPendingDeletePaths, InConfigFiles, bUseSourceControlStateCache);
}

bool FSourceControlWindows::PromptForCheckedOut(bool bUseSourceControlStateCache, const TArray<FString>& InFileNames, const FCheckedOutSetupInfo& InSetupInfo)
{
	TArray<FSourceControlStateRef> Items;

	if (InFileNames.Num() > 0)
	{
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

		if (!bUseSourceControlStateCache)
		{
			SourceControlProvider.Execute(ISourceControlOperation::Create<FUpdateStatus>(), InFileNames);
		}

		SourceControlProvider.GetState(InFileNames, Items, EStateCacheUsage::Use);
	}

	TSharedRef<SWindow> Window = SNew(SWindow)
		.SizingRule(ESizingRule::Autosized)
		.ClientSize(FVector2D(800, 600))
		.SupportsMaximize(false)
		.SupportsMinimize(false);

	TSharedRef<SSourceControlCheckedOutDialog> Widget =
		SNew(SSourceControlCheckedOutDialog)
		.ParentWindow(Window)
		.Items(Items)
		.ShowColumnAssetName(InSetupInfo.bShowColumnAssetName)
		.ShowColumnAssetClass(InSetupInfo.bShowColumnAssetClass)
		.ShowColumnUserName(InSetupInfo.bShowColumnUserName)
		.MessageText(InSetupInfo.MessageText)
		.CloseText(InSetupInfo.CloseText)
		.CheckBoxText(InSetupInfo.CheckboxText);

	Window->SetTitle(InSetupInfo.TitleText);
	Window->SetContent(
		Widget
	);

	FSlateApplication::Get().AddModalWindow(Window, NULL);

	return Widget->IsCheckBoxChecked();
}

// Note that:
// - FSourceControlWindows::DisplayRevisionHistory() is defined in SSourceControlHistory.cpp
// - FSourceControlWindows::PromptForRevert() is defined in SSourceControlRevert.cpp


void FSourceControlWindows::ChoosePackagesToCheckInCompleted(const TArray<UPackage*>& LoadedPackages, const TArray<FString>& PackageNames, const TArray<FString>& ConfigFiles, FCheckinResultInfo& OutResultInfo)
{
	if (ChoosePackagesToCheckInNotification.IsValid())
	{
		ChoosePackagesToCheckInNotification.Pin()->ExpireAndFadeout();
	}
	ChoosePackagesToCheckInNotification.Reset();

	// Prompt the user to ask if they would like to first save any dirty packages they are trying to check-in
	const FEditorFileUtils::EPromptReturnCode UserResponse = FEditorFileUtils::PromptForCheckoutAndSave(LoadedPackages, true, true);

	// If the user elected to save dirty packages, but one or more of the packages failed to save properly OR if the user
	// canceled out of the prompt, don't follow through on the check-in process
	const bool bShouldProceed = (UserResponse == FEditorFileUtils::EPromptReturnCode::PR_Success || UserResponse == FEditorFileUtils::EPromptReturnCode::PR_Declined);
	if (!bShouldProceed)
	{
		// If a failure occurred, alert the user that the check-in was aborted. This warning shouldn't be necessary if the user cancelled
		// from the dialog, because they obviously intended to cancel the whole operation.
		if (UserResponse == FEditorFileUtils::EPromptReturnCode::PR_Failure)
		{
			OutResultInfo.Description = NSLOCTEXT("UnrealEd", "SCC_Checkin_Aborted", "Check-in aborted as a result of save failure.");

			FMessageDialog::Open(EAppMsgType::Ok, OutResultInfo.Description);
		}

		return;
	}

	bool bUseSourceControlStateCache = true;
	TArray<FString> PendingDeletePaths = SourceControlHelpers::GetSourceControlLocations();

	PromptForCheckin(OutResultInfo, PackageNames, PendingDeletePaths, ConfigFiles, bUseSourceControlStateCache);
}

void FSourceControlWindows::ChoosePackagesToCheckInCancelled(FSourceControlOperationRef InOperation)
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	SourceControlProvider.CancelOperation(InOperation);

	if (ChoosePackagesToCheckInNotification.IsValid())
	{
		ChoosePackagesToCheckInNotification.Pin()->ExpireAndFadeout();
	}
	ChoosePackagesToCheckInNotification.Reset();
}

void FSourceControlWindows::ChoosePackagesToCheckInCallback(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult, FSourceControlWindowsOnCheckInComplete OnCompleteDelegate)
{
	if (ChoosePackagesToCheckInNotification.IsValid())
	{
		ChoosePackagesToCheckInNotification.Pin()->ExpireAndFadeout();
	}
	ChoosePackagesToCheckInNotification.Reset();

	FCheckinResultInfo ResultInfo;

	if (InResult != ECommandResult::Succeeded)
	{
		switch (InResult)
		{
			case ECommandResult::Cancelled:
				ResultInfo.Result      = ECommandResult::Cancelled;
				ResultInfo.Description = LOCTEXT("CheckInCancelled", "Check in cancelled.");
				break;

			case ECommandResult::Failed:
			{
				ResultInfo.Description = LOCTEXT("CheckInOperationFailed", "Failed checking revision control status!");
				FMessageLog EditorErrors("EditorErrors");
				EditorErrors.Warning(ResultInfo.Description);
				EditorErrors.Notify();
			}
		}

		OnCompleteDelegate.ExecuteIfBound(ResultInfo);

		return;
	}

	// Get a list of all the checked out packages
	TArray<FString> PackageNames;
	TArray<UPackage*> LoadedPackages;
	TMap<FString, FSourceControlStatePtr> PackageStates;
	FEditorFileUtils::FindAllSubmittablePackageFiles(PackageStates, true);

	TArray<FString> ConfigFilesToSubmit;

	for (TMap<FString, FSourceControlStatePtr>::TConstIterator PackageIter(PackageStates); PackageIter; ++PackageIter)
	{
		const FString PackageName = *PackageIter.Key();

		UPackage* Package = FindPackage(nullptr, *PackageName);
		if (Package != nullptr)
		{
			LoadedPackages.Add(Package);
		}

		PackageNames.Add(PackageName);
	}

	// Get a list of all the checked out project files
	TMap<FString, FSourceControlStatePtr> ProjectFileStates;
	FEditorFileUtils::FindAllSubmittableProjectFiles(ProjectFileStates);
	for (TMap<FString, FSourceControlStatePtr>::TConstIterator It(ProjectFileStates); It; ++It)
	{
		ConfigFilesToSubmit.Add(It.Key());
	}

	// Get a list of all the checked out config files
	TMap<FString, FSourceControlStatePtr> ConfigFileStates;
	FEditorFileUtils::FindAllSubmittableConfigFiles(ConfigFileStates);
	for (TMap<FString, FSourceControlStatePtr>::TConstIterator It(ConfigFileStates); It; ++It)
	{
		ConfigFilesToSubmit.Add(It.Key());
	}

	ChoosePackagesToCheckInCompleted(LoadedPackages, PackageNames, ConfigFilesToSubmit, ResultInfo);
	OnCompleteDelegate.ExecuteIfBound(ResultInfo);
}


#undef LOCTEXT_NAMESPACE

#endif // SOURCE_CONTROL_WITH_SLATE

