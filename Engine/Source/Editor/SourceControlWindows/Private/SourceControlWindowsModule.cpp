// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "ISourceControlWindowsModule.h"
#include "ISourceControlModule.h"
#include "SourceControlProviders.h"

#include "Widgets/Docking/SDockTab.h"
#include "Textures/SlateIcon.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"
#include "LevelEditor.h"

#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#include "SSourceControlChangelists.h"
#include "UncontrolledChangelistsModule.h"

#define LOCTEXT_NAMESPACE "SourceControlWindows"

/**
 * SourceControlWindows module
 */
class FSourceControlWindowsModule : public ISourceControlWindowsModule
{
public:
	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule() override;

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule() override;

	virtual void ShowChangelistsTab() override;
	virtual bool CanShowChangelistsTab() const override;

	virtual void ShowSnapshotHistoryTab() override;
	virtual bool CanShowSnapshotHistoryTab() const override;

	virtual void ShowConflictResolutionTab() override;
	virtual bool CanShowConflictResolutionTab() const override;

	virtual void ShowBranchExplorerTab() override;
	virtual bool CanShowBranchExplorerTab() const override;
	virtual bool IsVisibleBranchExplorerTab() const override;

	virtual void SelectFiles(const TArray<FString>& Filenames);

	DECLARE_DERIVED_EVENT(FSourceControlWindowsModule, ISourceControlWindowsModule::FChangelistFileDoubleClickedEvent, FChangelistFileDoubleClickedEvent);
	virtual FChangelistFileDoubleClickedEvent& OnChangelistFileDoubleClicked() override { return ChangelistFileDoubleClickedEvent; }

private:
	TSharedRef<SDockTab> CreateChangelistsTab(const FSpawnTabArgs& Args);
	TSharedPtr<SWidget> CreateChangelistsUI();

private:
	TWeakPtr<SDockTab> ChangelistsTab;
	TWeakPtr<SSourceControlChangelistsWidget> ChangelistsWidget;

	FChangelistFileDoubleClickedEvent ChangelistFileDoubleClickedEvent;
};

IMPLEMENT_MODULE(FSourceControlWindowsModule, SourceControlWindows);

static const FName SourceControlChangelistsTabName = FName(TEXT("SourceControlChangelists"));

void FSourceControlWindowsModule::StartupModule()
{
	ISourceControlWindowsModule::StartupModule();

	// Create a Source Control group under the Tools category
	const FSlateIcon SourceControlIcon(FAppStyle::GetAppStyleSetName(), "SourceControl.ChangelistsTab");

	// Register the changelist tab spawner
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(SourceControlChangelistsTabName, FOnSpawnTab::CreateRaw(this, &FSourceControlWindowsModule::CreateChangelistsTab))
		.SetDisplayName(LOCTEXT("ChangelistsTabTitle", "View Changes"))
		.SetTooltipText(LOCTEXT("ChangelistsTabTooltip", "Opens a dialog displaying current changes."))
		.SetIcon(SourceControlIcon)
		.SetMenuType(ETabSpawnerMenuType::Hidden);

#if WITH_RELOAD
	// This code attempts to relaunch the GameplayCueEditor tab when you reload this module
	if (IsReloadActive() && FSlateApplication::IsInitialized())
	{
		ShowChangelistsTab();
	}
#endif // WITH_RELOAD
}

void FSourceControlWindowsModule::ShutdownModule()
{
	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(SourceControlChangelistsTabName);

		if (ChangelistsTab.IsValid())
		{
			ChangelistsTab.Pin()->RequestCloseTab();
		}
	}
}

TSharedRef<SDockTab> FSourceControlWindowsModule::CreateChangelistsTab(const FSpawnTabArgs & Args)
{
	return SAssignNew(ChangelistsTab, SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			CreateChangelistsUI().ToSharedRef()
		];
}

TSharedPtr<SWidget> FSourceControlWindowsModule::CreateChangelistsUI()
{
	TSharedPtr<SWidget> ReturnWidget;
	if (IsInGameThread())
	{
		TSharedPtr<SSourceControlChangelistsWidget> SharedPtr = SNew(SSourceControlChangelistsWidget);
		ReturnWidget = SharedPtr;
		ChangelistsWidget = SharedPtr;
	}
	return ReturnWidget;
}

void FSourceControlWindowsModule::ShowChangelistsTab()
{
	FGlobalTabmanager::Get()->TryInvokeTab(FTabId(SourceControlChangelistsTabName));
}

bool FSourceControlWindowsModule::CanShowChangelistsTab() const
{
	ISourceControlModule& SourceControlModule = ISourceControlModule::Get();

	return (SourceControlModule.IsEnabled() && SourceControlModule.GetProvider().IsAvailable() && SourceControlModule.GetProvider().UsesChangelists()) || (FUncontrolledChangelistsModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().UsesUncontrolledChangelists());
}

void FSourceControlWindowsModule::ShowSnapshotHistoryTab()
{
	if (IConsoleObject* CObj = IConsoleManager::Get().FindConsoleObject(TEXT("UnrealRevisionControl.FocusSnapshotHistory")))
	{
		CObj->AsCommand()->Execute(/*Args=*/TArray<FString>(), /*InWorld=*/nullptr, *GLog);
	}
}

bool FSourceControlWindowsModule::CanShowSnapshotHistoryTab() const
{
	ISourceControlModule& SourceControlModule = ISourceControlModule::Get();

	return (SourceControlModule.IsEnabled() && SourceControlModule.GetProvider().IsAvailable() && SourceControlModule.GetProvider().GetName() == SourceControlProviders::GetUrcProviderName());
}

void FSourceControlWindowsModule::ShowConflictResolutionTab()
{
	if (IConsoleObject* CObj = IConsoleManager::Get().FindConsoleObject(TEXT("UnrealRevisionControl.FocusConflictResolution")))
	{
		CObj->AsCommand()->Execute(/*Args=*/TArray<FString>(), /*InWorld=*/nullptr, *GLog);
	}
}

bool FSourceControlWindowsModule::CanShowConflictResolutionTab() const
{
	ISourceControlModule& SourceControlModule = ISourceControlModule::Get();

	return (SourceControlModule.IsEnabled() && SourceControlModule.GetProvider().IsAvailable() && SourceControlModule.GetProvider().GetName() == SourceControlProviders::GetUrcProviderName());
}

void FSourceControlWindowsModule::ShowBranchExplorerTab()
{
	if (IConsoleObject* CObj = IConsoleManager::Get().FindConsoleObject(TEXT("UnrealRevisionControl.FocusBranchExplorer")))
	{
		CObj->AsCommand()->Execute(/*Args=*/TArray<FString>(), /*InWorld=*/nullptr, *GLog);
	}
}

bool FSourceControlWindowsModule::CanShowBranchExplorerTab() const
{
	ISourceControlModule& SourceControlModule = ISourceControlModule::Get();

	return (SourceControlModule.IsEnabled() && SourceControlModule.GetProvider().IsAvailable() && SourceControlModule.GetProvider().GetName() == SourceControlProviders::GetUrcProviderName());
}

bool FSourceControlWindowsModule::IsVisibleBranchExplorerTab() const
{
	if (IConsoleManager::Get().FindConsoleObject(TEXT("UnrealRevisionControl.FocusBranchExplorer")) == nullptr)
	{
		return false;
	}
	
	ISourceControlModule& SourceControlModule = ISourceControlModule::Get();

	return (SourceControlModule.IsEnabled() && SourceControlModule.GetProvider().IsAvailable() && SourceControlModule.GetProvider().GetName() == SourceControlProviders::GetUrcProviderName());
}

void FSourceControlWindowsModule::SelectFiles(const TArray<FString>& Filenames)
{
	if (Filenames.Num() > 0 && CanShowChangelistsTab())
	{
		ShowChangelistsTab();
		if (TSharedPtr<SSourceControlChangelistsWidget> ChangelistsWidgetPtr = ChangelistsWidget.Pin())
		{
			ChangelistsWidgetPtr->SetSelectedFiles(Filenames);
		}
	}
}

#undef LOCTEXT_NAMESPACE
