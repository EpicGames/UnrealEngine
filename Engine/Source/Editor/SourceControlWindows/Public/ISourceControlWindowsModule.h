// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Containers/Union.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

/**
 * SourceControlWindows module interface
 */

enum class FSubmitOverrideReply
{
	Handled,
	ProviderNotSupported,
	Error,

	Num
};

struct SSubmitOverrideParameters
{
	TUnion<FString, TArray<FString>> ToSubmit;
	FString Description;
};

struct FSubmitTitleOverrideParameters
{
	TUnion<FString, TArray<FString>> ToSubmit;
};

DECLARE_DELEGATE_RetVal_TwoParams(FSubmitOverrideReply, FCanSubmitOverrideDelegate, const SSubmitOverrideParameters& /*InParameters*/, FText* /*ErrorMessageOut*/);
DECLARE_DELEGATE_RetVal_OneParam(FSubmitOverrideReply, FSubmitOverrideDelegate, const SSubmitOverrideParameters& /*InParameters*/);
DECLARE_DELEGATE_RetVal_OneParam(FText, FSubmitTitleOverrideDelegate, const FSubmitTitleOverrideParameters& /*InParameters*/);

class ISourceControlWindowsModule : public IModuleInterface
{
public:
	/**
	 * Get reference to the SourceControlWindows module instance
	 */
	static inline ISourceControlWindowsModule& Get()
	{
		return FModuleManager::LoadModuleChecked<ISourceControlWindowsModule>("SourceControlWindows");
	}

	static inline ISourceControlWindowsModule* TryGet()
	{
		return FModuleManager::GetModulePtr<ISourceControlWindowsModule>("SourceControlWindows");
	}

	virtual void ShowChangelistsTab() = 0;
	virtual bool CanShowChangelistsTab() const = 0;

	virtual void ShowSnapshotHistoryTab() = 0;
	virtual bool CanShowSnapshotHistoryTab() const = 0;

	virtual void ShowConflictResolutionTab() = 0;
	virtual bool CanShowConflictResolutionTab() const = 0;

	virtual void ShowBranchExplorerTab() = 0;
	virtual bool CanShowBranchExplorerTab() const = 0;
	virtual bool IsVisibleBranchExplorerTab() const = 0;

	virtual void SelectFiles(const TArray<FString>& Filenames) = 0;

	DECLARE_EVENT_OneParam(ISourceControlWindowsModule, FChangelistFileDoubleClickedEvent, const FString&);
	virtual FChangelistFileDoubleClickedEvent& OnChangelistFileDoubleClicked() = 0;

	FCanSubmitOverrideDelegate CanSubmitOverrideDelegate;
	FSubmitOverrideDelegate SubmitOverrideDelegate;

	FSubmitTitleOverrideDelegate SubmitTitleOverrideDelegate;
};
