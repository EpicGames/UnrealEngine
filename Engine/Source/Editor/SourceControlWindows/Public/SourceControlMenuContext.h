// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ISourceControlChangelist.h"

#include "SourceControlMenuContext.generated.h"

/**
 * Source control window menu context providing information for menu extenders.
 */
UCLASS(MinimalAPI)
class USourceControlMenuContext : public UObject
{
	GENERATED_BODY()
public:

	USourceControlMenuContext(const FObjectInitializer& ObjectInitializer)
		: Super(ObjectInitializer)
	{
		
	}

	UPROPERTY()
	TArray<FString> SelectedFiles;

	FSourceControlChangelistPtr SelectedChangelist;
};
