// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "ISourceControlProvider.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "SourceControlWindows.generated.h"

#define UE_API SOURCECONTROLWINDOWS_API

class SSourceControlHistoryWidget;

/** Context object for SSourceControlHistoryWidget's right-click context menu to get info about which history widget is trying to generate the menu */
UCLASS(MinimalAPI)
class USourceControlHistoryWidgetContext : public UObject
{
	GENERATED_BODY()

public:
	struct SelectedItem
	{
		FString FileName;
		FString Revision;
	};

	TWeakPtr<class SSourceControlHistoryWidget> HistoryWidget;

	TArray<SelectedItem>& GetSelectedItems()
	{
		return SelectedItems;
	}

private:
	TArray<SelectedItem> SelectedItems;
};

/** Context object for SSourceControlSubmitWidget's right-click context menu to get info about which submit widget is trying to generate the menu */
UCLASS(MinimalAPI)
class USourceControlSubmitWidgetContext : public UObject
{
	GENERATED_BODY()

public:
	struct SelectedItem
	{
		FString FileName;
	};

	TWeakPtr<class SSourceControlSubmitWidget> SubmitWidget;

	TArray<SelectedItem>& GetSelectedItems()
	{
		return SelectedItems;
	}

private:
	TArray<SelectedItem> SelectedItems;
};

/** Info supplied as argument to delegate FSourceControlWindowsOnCheckInComplete called by FSourceControlWindows::ChoosePackagesToCheckIn() and optional argument to PromptForCheckin(). */
struct FCheckinResultInfo
{
	/** Default Constructor */
	UE_API FCheckinResultInfo();

	/** Succeeded - if packages were selected and successfully checked in, Cancelled - if the user aborted the process, Failed - if an issue was encountered during the process */
	ECommandResult::Type Result;

	/** true if added and modified files were automatically checked out from source control again after being submitted, false if not */
	bool bAutoCheckedOut;

	/** Files that were added. */
	TArray<FString> FilesAdded;

	/** Files that were modified and checked in. */
	TArray<FString> FilesSubmitted;

	/** Text that describes result whether failed, cancelled or successful. */
	FText Description;
};

/** Optional delegate called when FSourceControlWindows::ChoosePackagesToCheckIn() completes. */
DECLARE_DELEGATE_OneParam(FSourceControlWindowsOnCheckInComplete, const FCheckinResultInfo&);

class FSourceControlWindows
{
public:
	/**
	 * Opens a user dialog to choose packages to submit.
	 *
	 * @param	OnCompleteDelegate	Delegate to call when this user-based operation is complete. Also see FCheckinResultInfo.
	 * @return	true - if command successfully in progress and OnCompleteDelegate will be called when complete, false - if immediately unable to comply (such as source control not enabled)
	 */
	static UE_API bool ChoosePackagesToCheckIn(const FSourceControlWindowsOnCheckInComplete& OnCompleteDelegate = FSourceControlWindowsOnCheckInComplete());

	/** Determines whether we can choose packages to check in (we can't if an operation is already in progress) */
	static UE_API bool CanChoosePackagesToCheckIn();

	/** Determines if the Submit Content action should be visible or not */
	UE_DEPRECATED(5.6, "ShouldChoosePackagesToCheckBeVisible is deprecated and will be removed")
	static bool ShouldChoosePackagesToCheckBeVisible() { return false; }
	
	/**
	 * Saves all unsaved levels and assets and then - conditionally - performs an FSync operation to get latest and reloads the world
	 * 
	 * @return	true - if command completed successfully.
	 */
	static UE_API bool SyncLatest();

	/**
	 * Saves all unsaved levels and assets and then - conditionally - performs an FSync operation to get the specified revision, and reloads the world
	 *
	 * @return	true - if command completed successfully.
	 */
	static UE_API bool SyncRevision(const FString& InRevision);

	/** Determines whether we can sync to latest */
	static UE_API bool CanSyncLatest();

	/**
	 * Display check in dialog for the specified packages and get additional result information
	 *
	 * @param	OutResultInfo					Optional FCheckinResultInfo to store information about checked in files and whether the check in was successful
	 * @param	InPackageNames					Names of packages to check in
	 * @param	InPendingDeletePaths			Directories to check for files marked 'pending delete'
	 * @param	InConfigFiles					Config filenames to check in
	 * @param	bUseSourceControlStateCache		Whether to use the cached source control status, or force the status to be updated
	 * 
	 * @return	true - if completed successfully in progress and OnCompleteDelegate will be called when complete, false - if immediately unable to comply (such as source control not enabled)
	 */
	static UE_API bool PromptForCheckin(FCheckinResultInfo& OutResultInfo, const TArray<FString>& InPackageNames, const TArray<FString>& InPendingDeletePaths = TArray<FString>(), const TArray<FString>& InConfigFiles = TArray<FString>(), bool bUseSourceControlStateCache = false);

	/**
	 * Display check in dialog for the specified packages
	 *
	 * @param	bUseSourceControlStateCache		Whether to use the cached source control status, or force the status to be updated
	 * @param	InPackageNames					Names of packages to check in
	 * @param	InPendingDeletePaths			Directories to check for files marked 'pending delete'
	 * @param	InConfigFiles					Config filenames to check in
	 * 
	 * @return	true - if completed successfully in progress and OnCompleteDelegate will be called when complete, false - if immediately unable to comply (such as source control not enabled)
	 */
	static UE_API bool PromptForCheckin(bool bUseSourceControlStateCache, const TArray<FString>& InPackageNames, const TArray<FString>& InPendingDeletePaths = TArray<FString>(), const TArray<FString>& InConfigFiles = TArray<FString>());

	/**
	 * Display checked out dialog for the specified filenames
	 * @param	bUseSourceControlStateCache		Whether to use the cached source control status, or force the status to be updated
	 * @param	InFileNames						Filenames of files that are checked out
	 * @param	InSetupInfo						Config options
	 * 
	 * @return	true - if the dialog was closed with the optional checkbox checked
	 */
	struct FCheckedOutSetupInfo
	{
		FText TitleText;
		FText MessageText;
		FText CloseText;
		FText CheckboxText;

		bool bShowColumnAssetName = true;
		bool bShowColumnAssetClass = true;
		bool bShowColumnUserName = true;
	};

	static UE_API bool PromptForCheckedOut(bool bUseSourceControlStateCache, const TArray<FString>& InFileNames, const FCheckedOutSetupInfo& InSetupInfo);

	/**
	 * Display file revision history for the provided packages
	 *
	 * @param	InPackageNames	Names of packages to display file revision history for
	 */
	static UE_API void DisplayRevisionHistory( const TArray<FString>& InPackagesNames );

	/**
	 * Prompt the user with a revert files dialog, allowing them to specify which packages, if any, should be reverted.
	 *
	 * @param	InPackageNames	Names of the packages to consider for reverting
	 * @param	bInReloadWorld	Reload the world as part of the revert operation
	 *
	 * @return	true if the files were reverted; false if the user canceled out of the dialog
	 */
	static UE_API bool PromptForRevert(const TArray<FString>& InPackageNames, bool bInReloadWorld = false );

	/**
	 * Revert all locally modified files, and reload the world
	 */
	static UE_API bool RevertAllChangesAndReloadWorld();

	/**
	 * Displays file diff against workspace version
	 * 
	 * @param InFileName Name of the file to diff
	 * 
	 * @return true if the diff could be performed
	 */
	static UE_API bool DiffAgainstWorkspace(const FString& InFileName);

	/**
	 * Displays file diff against shelved file
	 * 
	 * @param InFileState SCC file state of the file to diff
	 * 
	 * @return true if the diff could be performed
	 */
	static UE_API bool DiffAgainstShelvedFile(const FSourceControlStateRef& InFileState);

protected:
	/** Callback for ChoosePackagesToCheckIn(), continues to bring up UI once source control operations are complete */
	static UE_API void ChoosePackagesToCheckInCallback(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult, FSourceControlWindowsOnCheckInComplete OnCompleteDelegate);

	/** Called when the user selection process has completed and we have packages to check in. */
	static UE_API void ChoosePackagesToCheckInCompleted(const TArray<UPackage*>& LoadedPackages, const TArray<FString>& PackageNames, const TArray<FString>& ConfigFiles, FCheckinResultInfo& OutResultInfo);

	/** Delegate called when the user has decided to cancel the check in process */
	static UE_API void ChoosePackagesToCheckInCancelled(FSourceControlOperationRef InOperation);

private:
	/** The notification in place while we choose packages to check in */
	static UE_API TWeakPtr<class SNotificationItem> ChoosePackagesToCheckInNotification;
};

#undef UE_API
