// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class SourceControlWindows : ModuleRules
{
	public SourceControlWindows(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core", 
				"CoreUObject", 
                "InputCore",
				"Engine",
				"Projects",
				"Slate",
				"SlateCore",
				"SourceControl", 
				"UncontrolledChangelists",
				"UnsavedAssetsTracker",
				"AssetDefinition",
				"AssetTools",
				"ToolWidgets",
				"EditorFramework",
				"WorkspaceMenuStructure",
				"UnrealEd",		// We need this dependency here because we use PackageTools.
			}
		);

		if(Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ToolMenus"
				}
			);
		}
	}
}
